// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetEditor_TaggedAssetBrowserConfiguration.h"

#include "TaggedAssetBrowserConfigurationToolkit.h"

void UAssetEditor_TaggedAssetBrowserConfiguration::SetObjectToEdit(UObject* InObject)
{
	ObjectToEdit = InObject;
}

void UAssetEditor_TaggedAssetBrowserConfiguration::GetObjectsToEdit(TArray<UObject*>& OutObjectsToEdit)
{
	OutObjectsToEdit.Add(ObjectToEdit);
}

TSharedPtr<FBaseAssetToolkit> UAssetEditor_TaggedAssetBrowserConfiguration::CreateToolkit()
{
	using namespace UE::UserAssetTags::AssetEditor;
	
	return MakeShared<FTaggedAssetBrowserConfigurationToolkit>(this);
}
