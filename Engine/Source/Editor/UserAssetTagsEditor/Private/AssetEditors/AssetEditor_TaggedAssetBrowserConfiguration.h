// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tools/UAssetEditor.h"
#include "AssetEditor_TaggedAssetBrowserConfiguration.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

/**
 * 
 */
UCLASS(MinimalAPI, Transient)
class UAssetEditor_TaggedAssetBrowserConfiguration : public UAssetEditor
{
	GENERATED_BODY()

public:
	UE_API void SetObjectToEdit(UObject* InObject);

protected:
	UE_API virtual void GetObjectsToEdit(TArray<UObject*>& OutObjectsToEdit) override;
	UE_API virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;

private:
	UPROPERTY(Transient)
	TObjectPtr<UObject> ObjectToEdit;
};

#undef UE_API
