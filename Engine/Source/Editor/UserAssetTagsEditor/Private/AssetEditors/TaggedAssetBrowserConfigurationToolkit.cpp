// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaggedAssetBrowserConfigurationToolkit.h"

#include "TaggedAssetBrowserMenuFilters.h"
#include "UserAssetTagEditorUtilities.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "DataHierarchyEditor/TaggedAssetBrowserConfigurationHierarchyViewModel.h"
#include "Tools/UAssetEditor.h"
#include "Widgets/SDataHierarchyEditor.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UserAssetTags"

namespace UE::UserAssetTags::AssetEditor
{
	const FName FTaggedAssetBrowserConfigurationToolkit::ToolkitFName(TEXT("TaggedAssetBrowserFilterHierarchyAssetEditor"));
	const FName FTaggedAssetBrowserConfigurationToolkit::HierarchyEditorTabId(TEXT("HierarchyEditorTabId"));
	
	FTaggedAssetBrowserConfigurationToolkit::FTaggedAssetBrowserConfigurationToolkit(UAssetEditor* InOwningAssetEditor) : FBaseAssetToolkit(InOwningAssetEditor)
	{
		using namespace UE::UserAssetTags::ViewModels;
		HierarchyViewModel.Reset(NewObject<UTaggedAssetBrowserConfigurationHierarchyViewModel>(GetTransientPackage()));

		TArray<UObject*> ObjectsToEdit;
		InOwningAssetEditor->GetObjectsToEdit(ObjectsToEdit);

		if(ensureMsgf(ObjectsToEdit.Num() == 1 && ObjectsToEdit[0]->IsA<UTaggedAssetBrowserConfiguration>(), TEXT("This toolkit supports requires a single TaggedAssetBrowserConfiguration object to function.")))
		{
			HierarchyViewModel->Initialize(*Cast<UTaggedAssetBrowserConfiguration>(ObjectsToEdit[0]));
		}
		
		StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_TaggedAssetBrowserConfiguration")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(			
				FTabManager::NewStack()
				->AddTab(HierarchyEditorTabId, ETabState::OpenedTab)
			)
		);
	}

	FTaggedAssetBrowserConfigurationToolkit::~FTaggedAssetBrowserConfigurationToolkit()
	{
		HierarchyViewModel->Finalize();
		HierarchyViewModel.Reset();
	}
	
	void FTaggedAssetBrowserConfigurationToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_TaggedAssetBrowserConfigurationEditor", "Tagged Filter Hierarchy Asset Editor"));

		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
		
		InTabManager->RegisterTabSpawner(HierarchyEditorTabId, FOnSpawnTab::CreateSP(this, &FTaggedAssetBrowserConfigurationToolkit::SpawnHierarchyEditorTab))
			.SetDisplayName(LOCTEXT("HierarchyEditorTab", "Hierarchy"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
	}

	void FTaggedAssetBrowserConfigurationToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

		InTabManager->UnregisterTabSpawner(HierarchyEditorTabId);
	}
	
	FName FTaggedAssetBrowserConfigurationToolkit::GetToolkitFName() const
	{
		return ToolkitFName;
	}

	FText FTaggedAssetBrowserConfigurationToolkit::GetBaseToolkitName() const
	{
		return LOCTEXT("AppLabel", "Tagged Asset Browser Configuration Editor");
	}

	FText FTaggedAssetBrowserConfigurationToolkit::GetToolkitToolTipText() const
	{
		return FAssetEditorToolkit::GetToolkitToolTipText();
	}

	FString FTaggedAssetBrowserConfigurationToolkit::GetWorldCentricTabPrefix() const
	{
		return LOCTEXT("WorldCentricTabPrefix", "Hierarchy").ToString();
	}

	FLinearColor FTaggedAssetBrowserConfigurationToolkit::GetWorldCentricTabColorScale() const
	{
		return FLinearColor(0.5, 0.5f, 0.8f);
	}
	
	TSharedRef<SDockTab> FTaggedAssetBrowserConfigurationToolkit::SpawnHierarchyEditorTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		check(SpawnTabArgs.GetTabId() == HierarchyEditorTabId);

		return SNew(SDockTab)
			.Label(LOCTEXT("HierarchyTabTitle", "Hierarchy"))
			.TabColorScale(GetTabColorScale())
			[
				SNew(SDataHierarchyEditor, HierarchyViewModel.Get())
				.OnGenerateRowContentWidget_Static(&GenerateFilterRowContent)
			];
	}

	TSharedRef<SWidget> FTaggedAssetBrowserConfigurationToolkit::GenerateFilterRowContent(TSharedRef<FHierarchyElementViewModel> HierarchyElementViewModel)
	{
		UHierarchyElement* HierarchyElement = HierarchyElementViewModel->GetDataMutable();
		if(UTaggedAssetBrowserFilterBase* Filter = Cast<UTaggedAssetBrowserFilterBase>(HierarchyElement))
		{
			TSharedPtr<SHorizontalBox> ExtensionBox;
			
			TSharedRef<SWidget> Widget = SNew(SHorizontalBox)
				.ToolTipText_UObject(Filter, &UTaggedAssetBrowserFilterBase::GetTooltip)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image_UObject(Filter, &UTaggedAssetBrowserFilterBase::GetIconBrush)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				[
					SNew(SHierarchyElement, HierarchyElementViewModel)
				]
				// The extension box takes up the rest of the space, so that users can determine left/right alignment
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SAssignNew(ExtensionBox, SHorizontalBox)
				];

			Filter->CreateAdditionalWidgets(ExtensionBox);

			return Widget;
		}

		return SNullWidget::NullWidget;
	}
}

#undef LOCTEXT_NAMESPACE
