// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataHierarchyViewModelBase.h"
#include "Toolkits/IToolkitHost.h"
#include "EditorUndoClient.h"
#include "Tools/BaseAssetToolkit.h"

class UTaggedAssetBrowserConfigurationHierarchyViewModel;
class UTaggedAssetBrowserConfiguration;

namespace UE::UserAssetTags::AssetEditor
{
	class FTaggedAssetBrowserConfigurationToolkit : public FBaseAssetToolkit
	{
	public:
		FTaggedAssetBrowserConfigurationToolkit(UAssetEditor* InOwningAssetBrowser);
		virtual ~FTaggedAssetBrowserConfigurationToolkit() override;

		virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
		
		virtual FName GetToolkitFName() const override;
		virtual FText GetBaseToolkitName() const override;
		virtual FText GetToolkitToolTipText() const override;
		virtual FString GetWorldCentricTabPrefix() const override;
		virtual FLinearColor GetWorldCentricTabColorScale() const override;
		virtual bool IsPrimaryEditor() const override { return true; }

	private:
		TSharedRef<class SDockTab> SpawnHierarchyEditorTab(const FSpawnTabArgs& SpawnTabArgs);

		TStrongObjectPtr<UTaggedAssetBrowserConfigurationHierarchyViewModel> HierarchyViewModel;

		static TSharedRef<SWidget> GenerateFilterRowContent(TSharedRef<FHierarchyElementViewModel> HierarchyElementViewModel);
		
		static const FName ToolkitFName;
		
		/**	The tab ids for all the tabs used */
		static const FName HierarchyEditorTabId;
	};
}
