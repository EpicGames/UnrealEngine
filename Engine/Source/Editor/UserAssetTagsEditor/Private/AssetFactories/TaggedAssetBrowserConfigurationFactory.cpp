// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetFactories/TaggedAssetBrowserConfigurationFactory.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"

UTaggedAssetBrowserConfigurationFactory::UTaggedAssetBrowserConfigurationFactory()
{
	SupportedClass = UTaggedAssetBrowserConfiguration::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
}

UObject* UTaggedAssetBrowserConfigurationFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UTaggedAssetBrowserConfiguration* NewAsset = NewObject<UTaggedAssetBrowserConfiguration>(InParent, InClass, InName, Flags | RF_Transactional);
	return NewAsset;
}
