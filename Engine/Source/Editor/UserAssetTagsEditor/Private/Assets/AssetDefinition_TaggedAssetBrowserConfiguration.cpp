// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_TaggedAssetBrowserConfiguration.h"

#include "AssetEditors/AssetEditor_TaggedAssetBrowserConfiguration.h"
#include "AssetEditors/TaggedAssetBrowserConfigurationToolkit.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Widgets/SDataHierarchyEditor.h"

#define LOCTEXT_NAMESPACE "TaggedAssetBrowser"

FText UAssetDefinition_TaggedAssetBrowserConfiguration::GetAssetDisplayName() const
{
	return LOCTEXT("HierarchyAssetDisplayName", "Tagged Asset Browser Configuration");
}

TSoftClassPtr<UObject> UAssetDefinition_TaggedAssetBrowserConfiguration::GetAssetClass() const
{
	return UTaggedAssetBrowserConfiguration::StaticClass();
}

FLinearColor UAssetDefinition_TaggedAssetBrowserConfiguration::GetAssetColor() const
{
	return FLinearColor(0.4f, 0.5f, 0.8f);
}

EAssetCommandResult UAssetDefinition_TaggedAssetBrowserConfiguration::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace UE::UserAssetTags::AssetEditor;

	if(OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		
		for (UTaggedAssetBrowserConfiguration* HierarchyAsset : OpenArgs.LoadObjects<UTaggedAssetBrowserConfiguration>())
		{
			UAssetEditor_TaggedAssetBrowserConfiguration* AssetEditor = NewObject<UAssetEditor_TaggedAssetBrowserConfiguration>(AssetEditorSubsystem, NAME_None, RF_Transient);

			AssetEditor->SetObjectToEdit(HierarchyAsset);
			AssetEditor->Initialize();
		}
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_TaggedAssetBrowserConfiguration::GetAssetCategories() const
{
	return Super::GetAssetCategories();
}

#undef LOCTEXT_NAMESPACE
