// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetDefinitionDefault.h"
#include "AssetDefinition_TaggedAssetBrowserConfiguration.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

/**
 * 
 */
UCLASS(MinimalAPI)
class UAssetDefinition_TaggedAssetBrowserConfiguration : public UAssetDefinitionDefault
{
	GENERATED_BODY()

	UE_API virtual FText GetAssetDisplayName() const override;
	UE_API virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	UE_API virtual FLinearColor GetAssetColor() const override;
	UE_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	UE_API virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
};

#undef UE_API
