// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/TaggedAssetBrowserConfiguration.h"

#include "UserAssetTagEditorUtilities.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/DataValidation.h"
#include "TaggedAssetBrowserFilters/TaggedAssetBrowser_CommonFilters.h"
#include "UObject/AssetRegistryTagsContext.h"

#define LOCTEXT_NAMESPACE "UserAssetTags"

UTaggedAssetBrowserConfiguration::UTaggedAssetBrowserConfiguration()
{
	FilterRoot = CreateDefaultSubobject<UTaggedAssetBrowserFilterRoot>("FilterRoot");
	StandaloneData.StandaloneFilterClasses = {
		UTaggedAssetBrowserFilter_All::StaticClass(),
		UTaggedAssetBrowserFilter_UserAssetTag::StaticClass(),
		UTaggedAssetBrowserFilter_UserAssetTagCollection::StaticClass(),
		UTaggedAssetBrowserFilter_Recent::StaticClass(),
		UTaggedAssetBrowserFilter_Directories::StaticClass(),
		UTaggedAssetBrowserFilter_Class::StaticClass()
	};
	StandaloneData.ExtensionFilterClasses = {
		UTaggedAssetBrowserFilter_UserAssetTag::StaticClass(),
		UTaggedAssetBrowserFilter_UserAssetTagCollection::StaticClass(),
		UTaggedAssetBrowserFilter_Directories::StaticClass(),
		UTaggedAssetBrowserFilter_Class::StaticClass()
	};
}

EDataValidationResult UTaggedAssetBrowserConfiguration::IsDataValid(class FDataValidationContext& Context) const
{
	if(ProfileName.IsValid() == false)
	{
		Context.AddWarning(LOCTEXT("Validation_InvalidProfileName", "No profile name was specified."));
	}
	else if(bIsExtension == false)
	{
		TArray<const UHierarchyElement*> AllElements;
		FilterRoot->GetChildrenOfType(AllElements, true);

		bool bAnyErrors = false;
		for(const UHierarchyElement* Element : AllElements)
		{
			if(StandaloneData.StandaloneFilterClasses.Contains(Element->GetClass()) == false)
			{
				Context.AddError(FText::FormatOrdered(LOCTEXT("Validation_InvalidElementTypeInStandalone", "Hierarchy Element {0} with type {1} is not allowed according to the standalone configuration. You are advised to delete it or add the type to StandaloneFilterClasses."), FText::FromString(Element->ToString()), Element->GetClass()->GetDisplayNameText()));
				bAnyErrors = true;
			}
		}

		if(bAnyErrors)
		{
			return EDataValidationResult::Invalid;
		}
	}
	// If this is an extension, check for the parent asset
	else if(bIsExtension && ProfileName.IsValid())
	{
		TArray<FAssetData> StandaloneConfigurationAssets = UE::UserAssetTags::FindStandaloneConfigurationForExtension(this);

		// If there is no parent asset, there probably was a typo
		if(StandaloneConfigurationAssets.Num() == 0)
		{
			Context.AddWarning(FText::FormatOrdered(LOCTEXT("Validation_InvalidStandaloneConfigurationForExtension", "No standalone configuration asset with name {0} was found. The Profile Name might contain a typo."), FText::FromName(ProfileName)));
			return EDataValidationResult::Invalid;
		}
		// If there is more than one, we won't know which is the 'active' one
		else if(StandaloneConfigurationAssets.Num() > 1)
		{
			// Not this asset's fault so we don't fail it.
			Context.AddWarning(FText::FormatOrdered(LOCTEXT("Validation_TooManyStandaloneConfigurationForExtension", "More than one standalone configuration asset with name {0} was found."), FText::FromName(ProfileName)));
		}
		// If we found exactly one matching standalone for this extension, check if all elements in our hierarchy are valid according to its extension rules
		else
		{
			UTaggedAssetBrowserConfiguration* StandaloneConfiguration = Cast<UTaggedAssetBrowserConfiguration>(StandaloneConfigurationAssets[0].GetAsset());

			TArray<const UHierarchyElement*> AllElements;
			FilterRoot->GetChildrenOfType(AllElements, true);
			
			for(const UHierarchyElement* Element : AllElements)
			{
				if(StandaloneConfiguration->StandaloneData.ExtensionFilterClasses.Contains(Element->GetClass()) == false)
				{
					Context.AddError(FText::FormatOrdered(LOCTEXT("Validation_InvalidFilterTypeInExtensionHierarchy", "Hierarchy Element {0} with type {1} is not allowed according to the standalone configuration. You are advised to delete it."), FText::FromString(Element->ToString()), Element->GetClass()->GetDisplayNameText()));
				}
			}

			for(const UHierarchySection* Section : FilterRoot->GetSectionData())
			{
				const UTaggedAssetBrowserSection* TaggedAssetBrowserSection = CastChecked<UTaggedAssetBrowserSection>(Section);
				for(const UTaggedAssetBrowserFilterBase* SectionFilter : TaggedAssetBrowserSection->Filters)
				{				
					if(SectionFilter != nullptr && StandaloneConfiguration->StandaloneData.ExtensionFilterClasses.Contains(SectionFilter->GetClass()) == false)
					{
						Context.AddError(FText::FormatOrdered(LOCTEXT("Validation_InvalidFilterTypeInExtensionSection", "Filter with type {0} in Section {1} is not allowed according to the standalone configuration. You are advised to delete it."), SectionFilter->GetClass()->GetDisplayNameText(), TaggedAssetBrowserSection->GetSectionNameAsText()));
					}
				}
			}
		}
	}

	TArray<const UTaggedAssetBrowserFilterBase*> AllFilters;

	for(const UHierarchySection* Section : FilterRoot->GetSectionData())
	{
		const UTaggedAssetBrowserSection* TaggedAssetBrowserSection = CastChecked<UTaggedAssetBrowserSection>(Section);
		for(const UTaggedAssetBrowserFilterBase* SectionFilter : TaggedAssetBrowserSection->Filters)
		{				
			if(SectionFilter == nullptr)
			{
				Context.AddError(FText::FormatOrdered(LOCTEXT("Validation_NullptrSectionFilter", "A filter member in section {0} was not properly set. Delete or set properly."), TaggedAssetBrowserSection->GetSectionNameAsText()));
			}
		}

		AllFilters.Append(TaggedAssetBrowserSection->Filters);
	}

	FilterRoot->GetChildrenOfType(AllFilters, true);
	
	for(const UTaggedAssetBrowserFilterBase* Filter : AllFilters)
	{
		if(Filter && Filter->GetClass()->HasAnyClassFlags(CLASS_Deprecated))
		{
			Context.AddWarning(FText::FormatOrdered(LOCTEXT("Validation_DeprecatedFilter", "Filter {0} is deprecated."), FText::FromString(Filter->ToString())));
		}
	}

	if(Context.GetNumErrors() > 0)
	{
		return EDataValidationResult::Invalid;
	}
	else if(Context.GetNumWarnings() > 0)
	{
		return EDataValidationResult::Valid;
	}
	
	return EDataValidationResult::Valid;
}

#undef LOCTEXT_NAMESPACE
