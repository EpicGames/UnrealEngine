// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/UserAssetTagCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "AssetEditorCommonCommands"

void FUserAssetTagCommands::RegisterCommands()
{
	UI_COMMAND(ManageTags, "Manage Tags", "Open a window to manage tags for all selected assets", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::T));
}

#undef LOCTEXT_NAMESPACE
