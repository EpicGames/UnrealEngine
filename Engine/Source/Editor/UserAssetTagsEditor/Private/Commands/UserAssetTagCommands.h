// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FUICommandInfo;

/** User Asset Tag Commands */
class FUserAssetTagCommands : public TCommands<FUserAssetTagCommands>
{

public:
	FUserAssetTagCommands()
		: TCommands<FUserAssetTagCommands>( TEXT("UserAssetTags"), NSLOCTEXT("Contexts", "UserAssetTags", "User Asset Tags"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}	

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> ManageTags;
};

