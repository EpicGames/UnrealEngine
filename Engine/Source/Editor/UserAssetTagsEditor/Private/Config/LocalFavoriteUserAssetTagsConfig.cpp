// Copyright Epic Games, Inc. All Rights Reserved.

#include "Config/LocalFavoriteUserAssetTagsConfig.h"

TStrongObjectPtr<ULocalFavoriteUserAssetTagsConfig> ULocalFavoriteUserAssetTagsConfig::Instance = nullptr;

ULocalFavoriteUserAssetTagsConfig* ULocalFavoriteUserAssetTagsConfig::Get()
{
	if(!Instance.IsValid())
	{
		Instance.Reset(NewObject<ULocalFavoriteUserAssetTagsConfig>());
		Instance->LoadEditorConfig();
	}
	
	return Instance.Get();
}

void ULocalFavoriteUserAssetTagsConfig::Shutdown()
{
	if(Instance != nullptr)
	{
		Instance->SaveEditorConfig();
		Instance.Reset();
	}
}

TSet<FName> ULocalFavoriteUserAssetTagsConfig::GetFavoriteUserAssetTagsForClass(const UClass* Class)
{
	FSoftClassPath ClassPath(Class);
	if(FavoriteUserAssetTagsPerClass.Contains(ClassPath))
	{
		return FavoriteUserAssetTagsPerClass[ClassPath].FavoriteUserAssetTags;
	}

	return {};
}

void ULocalFavoriteUserAssetTagsConfig::ToggleFavoriteUserAssetTag(const UClass* Class, FName InUserAssetTag)
{
	FSoftClassPath ClassPath(Class);
	if(FavoriteUserAssetTagsPerClass.Contains(ClassPath) && FavoriteUserAssetTagsPerClass[ClassPath].FavoriteUserAssetTags.Contains(InUserAssetTag))
	{
		FavoriteUserAssetTagsPerClass[ClassPath].FavoriteUserAssetTags.Remove(InUserAssetTag);
	}
	else
	{
		FavoriteUserAssetTagsPerClass.FindOrAdd(ClassPath).FavoriteUserAssetTags.Add(InUserAssetTag);
	}
}
