// Copyright Epic Games, Inc. All Rights Reserved.

#include "Config/TaggedAssetBrowserConfig.h"

#include "UObject/StrongObjectPtrTemplates.h"

TStrongObjectPtr<UTaggedAssetBrowserConfig> UTaggedAssetBrowserConfig::Instance = nullptr;

UTaggedAssetBrowserConfig* UTaggedAssetBrowserConfig::Get()
{	
	if(!Instance)
	{
		Instance.Reset(NewObject<UTaggedAssetBrowserConfig>());
		Instance->LoadEditorConfig();
	}
	
	return Instance.Get();
}

void UTaggedAssetBrowserConfig::Shutdown()
{
	if(Instance)
	{
		Instance->SaveEditorConfig();
		Instance.Reset();
	}
}

bool UTaggedAssetBrowserConfig::HasInstance()
{
	return Instance.IsValid();
}

void UTaggedAssetBrowserConfig::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnPropertyChangedDelegate.Broadcast(PropertyChangedEvent);
	SaveEditorConfig();
}
