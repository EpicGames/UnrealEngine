// Copyright Epic Games, Inc. All Rights Reserved.

#include "Config/UserAssetTagsEditorConfig.h"
#include "UserAssetTagProvider.h"

TStrongObjectPtr<UUserAssetTagsEditorConfig> UUserAssetTagsEditorConfig::Instance = nullptr;

void UUserAssetTagsEditorConfig::ToggleSortByAlphabet()
{
	bSortByAlphabet = !bSortByAlphabet;

	SaveEditorConfig();
}

bool UUserAssetTagsEditorConfig::ShouldSortByAlphabet() const
{
	return bSortByAlphabet;
}

bool UUserAssetTagsEditorConfig::IsProviderEnabled(const UClass* ProviderClass) const
{
	if(ProviderViewOptions.PerProviderViewOptions.Contains(ProviderClass->GetFName()))
	{
		return ProviderViewOptions.PerProviderViewOptions[ProviderClass->GetFName()].bEnabled;
	}
	
	return ProviderClass->GetDefaultObject<UUserAssetTagProvider>()->IsEnabledByDefault();
}

void UUserAssetTagsEditorConfig::ToggleProviderEnabled(const UClass* ProviderClass)
{
	if(ProviderViewOptions.PerProviderViewOptions.Contains(ProviderClass->GetFName()))
	{
		ProviderViewOptions.PerProviderViewOptions[ProviderClass->GetFName()].bEnabled = !ProviderViewOptions.PerProviderViewOptions[ProviderClass->GetFName()].bEnabled;
	}
	else
	{
		bool& bEnabled = ProviderViewOptions.PerProviderViewOptions.Add(ProviderClass->GetFName()).bEnabled;
		bEnabled = !ProviderClass->GetDefaultObject<UUserAssetTagProvider>()->IsEnabledByDefault();
	}

	SaveEditorConfig();
}

EUserAssetTagProviderMenuType UUserAssetTagsEditorConfig::GetProviderMenuType(const UClass* ProviderClass) const
{
	if(ProviderViewOptions.PerProviderViewOptions.Contains(ProviderClass->GetFName()))
	{
		return ProviderViewOptions.PerProviderViewOptions[ProviderClass->GetFName()].MenuType;
	}

	return ProviderClass->GetDefaultObject<UUserAssetTagProvider>()->GetMenuTypeByDefault();
}

void UUserAssetTagsEditorConfig::SetProviderMenuType(const UClass* ProviderClass, EUserAssetTagProviderMenuType InMenuType)
{
	if(ProviderViewOptions.PerProviderViewOptions.Contains(ProviderClass->GetFName()))
	{
		ProviderViewOptions.PerProviderViewOptions[ProviderClass->GetFName()].MenuType = InMenuType;
	}
	else
	{
		EUserAssetTagProviderMenuType& MenuType = ProviderViewOptions.PerProviderViewOptions.Add(ProviderClass->GetFName()).MenuType;
		MenuType = InMenuType;
	}

	SaveEditorConfig();
}

UUserAssetTagsEditorConfig* UUserAssetTagsEditorConfig::Get()
{
	if(Instance.IsValid() == false)
	{
		Instance.Reset(NewObject<UUserAssetTagsEditorConfig>());
		Instance->LoadEditorConfig();
	}

	return Instance.Get();
}

void UUserAssetTagsEditorConfig::Shutdown()
{
	if(Instance != nullptr)
	{
		Instance->SaveEditorConfig();
		Instance.Reset();
	}
}
