// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaggedAssetBrowserConfigurationCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "UserAssetTagEditorUtilities.h"
#include "Widgets/Input/SEditableComboBox.h"
#include "Widgets/Input/SSuggestionTextBox.h"
#include "Widgets/Input/STextComboBox.h"

FTaggedAssetBrowserConfigurationCustomization::~FTaggedAssetBrowserConfigurationCustomization()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void FTaggedAssetBrowserConfigurationCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> OutObjects;
	DetailBuilder.GetObjectsBeingCustomized(OutObjects);

	UObject* EditedObject = OutObjects[0].Get();
	ConfigurationAsset = Cast<UTaggedAssetBrowserConfiguration>(EditedObject);

	if(OnObjectPropertyChangeDelegateHandle.IsValid() == false)
	{
		OnObjectPropertyChangeDelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FTaggedAssetBrowserConfigurationCustomization::OnObjectPropertyChanged);
	}

	// For standalone assets, we do nothing
	if(ConfigurationAsset->bIsExtension == false)
	{
		return;
	}
	// For extensions, we use a text combo box that offers valid options for the profile names out of the box
	else
	{
		ProfileNameHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UTaggedAssetBrowserConfiguration, ProfileName));
		ProfileNameHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &FTaggedAssetBrowserConfigurationCustomization::OnProfileNameReset));
		IDetailPropertyRow& Row = *DetailBuilder.EditDefaultProperty(ProfileNameHandle);

		TSharedPtr<SWidget> DefaultNameWidget;
		TSharedPtr<SWidget> DefaultValueWidget;
		Row.GetDefaultWidgets(DefaultNameWidget, DefaultValueWidget, false);
		Row.CustomWidget(false);
		(*Row.CustomNameWidget())
		[
			DefaultNameWidget.ToSharedRef()
		];
		
		Algo::Transform(GetProfileNameSuggestionsForExtension(), AvailableProfileNames, [](const FString& Name)
		{
			return MakeShared<FString>(Name);
		});

		FString CurrentProfileName = GetCurrentProfileName().ToString();
		TSharedPtr<FString>* ExistingCurrentProfileNamePtr = AvailableProfileNames.FindByPredicate([this, CurrentProfileName](TSharedPtr<FString> Candidate)
		{
			return Candidate->Equals(CurrentProfileName);
		});

		// If the existing name already was in the list of options, we use that as our initial profile name
		if(ExistingCurrentProfileNamePtr)
		{
			InitialProfileName = *ExistingCurrentProfileNamePtr;
		}
		// If not, we insert it as option 0; this might be the case if the existing option was deleted post-fact.
		// We do this so that the combo box can reliably display the initial option
		else
		{
			InitialProfileName = MakeShared<FString>();
			*InitialProfileName = CurrentProfileName;
			AvailableProfileNames.Insert(InitialProfileName, 0);
		}
		
		(*Row.CustomValueWidget())
		[
			SNew(STextComboBox)
			.OptionsSource(&AvailableProfileNames)
			.InitiallySelectedItem(InitialProfileName)
			.OnSelectionChanged(STextComboBox::FOnTextSelectionChanged::CreateSP(this, &FTaggedAssetBrowserConfigurationCustomization::OnProfileNameSelectionChanged))
		];
	}
}

void FTaggedAssetBrowserConfigurationCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	DetaillLayoutBuilderWeak = DetailBuilder;
	IDetailCustomization::CustomizeDetails(DetailBuilder);
}

TArray<FString> FTaggedAssetBrowserConfigurationCustomization::GetProfileNameSuggestionsForExtension() const
{
	if(ConfigurationAsset.IsValid() == false)
	{
		return {};
	}

	if(ConfigurationAsset->bIsExtension)
	{
		TArray<FString> Result;
		Algo::Transform(UE::UserAssetTags::FindAllStandaloneConfigurationAssetProfileNames(), Result, [](const FName& Name)
		{
			return Name.ToString();
		});

		return Result;
	}

	return {};
}

FName FTaggedAssetBrowserConfigurationCustomization::GetCurrentProfileName() const
{
	FName Value;
	ProfileNameHandle->GetValue(Value);
	return Value;
}

void FTaggedAssetBrowserConfigurationCustomization::OnProfileNameSelectionChanged(TSharedPtr<FString> Text, ESelectInfo::Type SelectInfo) const
{
	if(Text.IsValid())
	{
		ProfileNameHandle->SetValue(FName(*Text));
	}
}

void FTaggedAssetBrowserConfigurationCustomization::OnProfileNameReset()
{
	DetaillLayoutBuilderWeak.Pin()->ForceRefreshDetails();
}

void FTaggedAssetBrowserConfigurationCustomization::OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if(Object->IsA<UTaggedAssetBrowserConfiguration>())
	{
		if(PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTaggedAssetBrowserConfiguration, bIsExtension))
		{
			DetaillLayoutBuilderWeak.Pin()->ForceRefreshDetails();
		}
	}
}
