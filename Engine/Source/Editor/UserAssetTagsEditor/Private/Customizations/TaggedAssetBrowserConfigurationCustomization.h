// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;
class SWidget;

class FTaggedAssetBrowserConfigurationCustomization : public IDetailCustomization
{
public:
	FTaggedAssetBrowserConfigurationCustomization() = default;
	virtual ~FTaggedAssetBrowserConfigurationCustomization() override;
	
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FTaggedAssetBrowserConfigurationCustomization>();
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;

protected:
	TArray<FString> GetProfileNameSuggestionsForExtension() const;
	FName GetCurrentProfileName() const;
	
	void OnProfileNameSelectionChanged(TSharedPtr<FString> Text, ESelectInfo::Type SelectInfo) const;
	void OnProfileNameReset();

	void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

private:
	TWeakPtr<IDetailLayoutBuilder> DetaillLayoutBuilderWeak;
	TWeakObjectPtr<UTaggedAssetBrowserConfiguration> ConfigurationAsset;
	TSharedPtr<IPropertyHandle> ProfileNameHandle;
	TSharedPtr<FString> InitialProfileName;
	TArray<TSharedPtr<FString>> AvailableProfileNames;
	FDelegateHandle OnObjectPropertyChangeDelegateHandle;
};
