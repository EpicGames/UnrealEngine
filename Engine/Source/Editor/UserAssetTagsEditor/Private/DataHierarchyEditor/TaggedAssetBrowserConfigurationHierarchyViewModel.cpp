// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataHierarchyEditor/TaggedAssetBrowserConfigurationHierarchyViewModel.h"
#include "TaggedAssetBrowserFilters/TaggedAssetBrowser_CommonFilters.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "UserAssetTagEditorUtilities.h"
#include "UserAssetTagsEditorModule.h"
#include "Logging/StructuredLog.h"

#define LOCTEXT_NAMESPACE "UserAssetTags"

void UTaggedAssetBrowserConfigurationHierarchyViewModel::Initialize(UTaggedAssetBrowserConfiguration& InAsset)
{
	ConfigurationAsset = &InAsset;
	
	Super::Initialize();		
}

void UTaggedAssetBrowserConfigurationHierarchyViewModel::InitializeInternal()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UTaggedAssetBrowserConfigurationHierarchyViewModel::EmptyCache);
}

void UTaggedAssetBrowserConfigurationHierarchyViewModel::FinalizeInternal()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

UHierarchyRoot* UTaggedAssetBrowserConfigurationHierarchyViewModel::GetHierarchyRoot() const
{
	if(ConfigurationAsset.IsValid())
	{
		return ConfigurationAsset->FilterRoot;
	}

	return nullptr;
}

TSharedPtr<FHierarchyElementViewModel> UTaggedAssetBrowserConfigurationHierarchyViewModel::CreateCustomViewModelForElement(UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> Parent)
{
	FUserAssetTagsEditorModule& UserAssetTagsEditorModule = FUserAssetTagsEditorModule::Get();
	if(UserAssetTagsEditorModule.GetConfigurationHierarchyViewModelFactories().Contains(Element->GetClass()))
	{
		return UserAssetTagsEditorModule.GetConfigurationHierarchyViewModelFactories()[Element->GetClass()].Execute(Element, Parent, this);
	}

	UE_LOGFMT(LogUserAssetTags, Display, "No custom view model was specified for hierarchy element type {0}. Falling back to default view model. Alternatively, you can register a view model using FUserAssetTagsEditorModule::RegisterConfigurationHierarchyElementViewModel.", Element->GetClass()->GetDisplayNameText().ToString());
	return nullptr;
}

TSubclassOf<UHierarchySection> UTaggedAssetBrowserConfigurationHierarchyViewModel::GetSectionDataClass() const
{
	return UTaggedAssetBrowserSection::StaticClass();
}

TArray<TSubclassOf<UHierarchyElement>> UTaggedAssetBrowserConfigurationHierarchyViewModel::GetAdditionalTypesToAddInUi() const
{
	TArray<TSubclassOf<UHierarchyElement>> Result;

	if(AdditionalTypesCache.IsSet() == false)
	{
		if(ConfigurationAsset->bIsExtension)
		{
			Algo::Transform(GetAdditionalTypesToAddInUi_Extension(), Result, [](TSubclassOf<UTaggedAssetBrowserFilterBase> Candidate)
			{
				return Candidate;	
			});
		}
		else
		{
			Algo::Transform(GetAdditionalTypesToAddInUi_Standalone(), Result, [](TSubclassOf<UTaggedAssetBrowserFilterBase> Candidate)
			{
				return Candidate;	
			});
		}

		AdditionalTypesCache = Result;
	}
	
	return AdditionalTypesCache.GetValue();
}

TArray<TSubclassOf<UTaggedAssetBrowserFilterBase>> UTaggedAssetBrowserConfigurationHierarchyViewModel::GetAdditionalTypesToAddInUi_Standalone() const
{	
	TArray<TSubclassOf<UTaggedAssetBrowserFilterBase>> Result = ConfigurationAsset->StandaloneData.StandaloneFilterClasses;
	
	Result.RemoveAll([](TSubclassOf<UTaggedAssetBrowserFilterBase> Candidate)
	{
		if(Candidate == nullptr)
		{
			return true;
		}
		
		if(Candidate->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
		{
			UE_LOGFMT(LogUserAssetTags, Warning, "Abstract or deprecated class found in list of valid default filter classes in {Candidate}", Candidate->GetName());
			return true;
		}

		return false;
	});

	return Result;
}

TArray<TSubclassOf<UTaggedAssetBrowserFilterBase>> UTaggedAssetBrowserConfigurationHierarchyViewModel::GetAdditionalTypesToAddInUi_Extension() const
{
	TArray<FAssetData> StandaloneConfigurationAssets = UE::UserAssetTags::FindStandaloneConfigurationForExtension(ConfigurationAsset.Get());

	if(StandaloneConfigurationAssets.Num() >= 1)
	{
		if(StandaloneConfigurationAssets.Num() > 1)
		{
			UE_LOGFMT(LogUserAssetTags, Warning, "Encountered multiple standalone configurations with profile name {0}. Selecting first one, which might be wrong.", ConfigurationAsset->ProfileName);
		}
		
		TArray<TSubclassOf<UTaggedAssetBrowserFilterBase>> Result = Cast<UTaggedAssetBrowserConfiguration>(StandaloneConfigurationAssets[0].GetAsset())->StandaloneData.ExtensionFilterClasses;
	
		Result.RemoveAll([](TSubclassOf<UTaggedAssetBrowserFilterBase> Candidate)
		{
			if(Candidate == nullptr)
			{
				return true;
			}
		
			if(Candidate->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
			{
				UE_LOGFMT(LogUserAssetTags, Warning, "Abstract or deprecated class found in list of valid extension filter classes in {0}", Candidate->GetName());
				return true;
			}

			return false;
		});

		return Result;
	}

	return {};
}

void UTaggedAssetBrowserConfigurationHierarchyViewModel::EmptyCache(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	// We empty out the cache whenever a configuration asset changed because we also need to react to changes to a standalone configuration with the same profile name
	if(Object->IsA<UTaggedAssetBrowserConfiguration>())
	{
		AdditionalTypesCache.Reset();
	}
}

namespace UE::UserAssetTags::ViewModels
{
	FHierarchyElementViewModel::FResultWithUserFeedback FFilterViewModel_Root::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone)
	{
		FResultWithUserFeedback Results(true);
		
		// We don't allow sections to be dropped on the root directly
		if(DraggedElement->GetData()->IsA<UHierarchySection>() && ItemDropZone == EItemDropZone::OntoItem)
		{
			return false;
		}
		
		FDataHierarchyElementMetaData_SectionAssociation SectionAssociation = DraggedElement->GetData()->FindMetaDataOfTypeOrDefault<FDataHierarchyElementMetaData_SectionAssociation>();
		if(SectionAssociation.Section != HierarchyViewModel->GetActiveHierarchySectionViewModel()->GetSectionData())
		{
			FText BaseMessage = LOCTEXT("RootDrop_SectionChange", "The drop will change the section for element {0} to {1}");
			Results.UserFeedback = FText::FormatOrdered(BaseMessage, DraggedElement->ToStringAsText(), HierarchyViewModel->GetActiveHierarchySectionViewModel()->GetSectionNameAsText());	
		}
		
		return Results;
	}

	UObject* FFilterViewModel_Root::GetObjectForEditing()
	{
		return Cast<UTaggedAssetBrowserConfigurationHierarchyViewModel>(GetHierarchyViewModel())->GetConfigurationAsset();
	}
	
	FFilterViewModel_All::FFilterViewModel_All(UTaggedAssetBrowserFilter_All* InFilter, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel)
		: FFilterViewModelBase(InFilter, InParent, InHierarchyViewModel)
	{
	}

	FFilterViewModel_Section::FFilterViewModel_Section(UTaggedAssetBrowserSection* InSection, TSharedRef<FHierarchyRootViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel)
		: FHierarchySectionViewModel(InSection, InParent, ViewModel)
	{
	}

	const FSlateBrush* FFilterViewModel_Section::GetSectionImageBrush() const
	{
		if(const UTaggedAssetBrowserSection* Section = GetData<UTaggedAssetBrowserSection>())
		{
			return Section->IconData.GetImageBrush();
		}

		return FAppStyle::GetNoBrush();
	}

	FHierarchyElementViewModel::FResultWithUserFeedback FFilterViewModel_Section::CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType)
	{
		if(InHierarchyElementType->IsChildOf<UHierarchySection>())
		{
			return false;
		}

		// Our sections can be associated with any element
		return true;
	}

	FFilterViewModelBase::FFilterViewModelBase(UTaggedAssetBrowserFilterBase* InFilter, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel)
		: FHierarchyItemViewModel(InFilter, InParent, InHierarchyViewModel)
	{
	}

	FFilterViewModel_UserAssetTag::FFilterViewModel_UserAssetTag(UTaggedAssetBrowserFilter_UserAssetTag* InFilter, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel)
		: FFilterViewModelBase(InFilter, InParent, InHierarchyViewModel)
	{
	}

	FHierarchyElementViewModel::FResultWithUserFeedback FFilterViewModel_UserAssetTag::CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType)
	{
		if(InHierarchyElementType->IsChildOf<UTaggedAssetBrowserFilter_UserAssetTag>())
		{
			return true;
		}

		FResultWithUserFeedback Results(false);
		Results.UserFeedback = LOCTEXT("TagCanContain_Denied", "Asset Tags can only contain other Asset Tags.");
		return Results;
	}

	void FFilterViewModel_UserAssetTag::RenameInternal(FName NewName)
	{
		GetDataMutable<UTaggedAssetBrowserFilter_UserAssetTag>()->SetUserAssetTag(NewName);
	}

	FFilterViewModel_UserAssetTagCollection::FFilterViewModel_UserAssetTagCollection(UTaggedAssetBrowserFilter_UserAssetTagCollection* InFilter, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel)
		: FFilterViewModelBase(InFilter, InParent, InHierarchyViewModel)
	{
	}

	FHierarchyElementViewModel::FResultWithUserFeedback FFilterViewModel_UserAssetTagCollection::CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType)
	{
		if(InHierarchyElementType->IsChildOf<UTaggedAssetBrowserFilter_UserAssetTag>())
		{
			return true;
		}

		FResultWithUserFeedback Results(false);
		Results.UserFeedback = LOCTEXT("TagCollectionCanContain_Denied", "Asset Tag Collections can only contain other Asset Tags.");
		return Results;
	}

	FHierarchyElementViewModel::FResultWithUserFeedback FFilterViewModel_UserAssetTagCollection::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedItem, EItemDropZone ItemDropZone)
	{
		if(DraggedItem->GetData()->IsA<UTaggedAssetBrowserFilter_UserAssetTag>())
		{
			FResultWithUserFeedback Results(true);
			return Results;
		}
		return FHierarchyItemViewModel::CanDropOnInternal(DraggedItem, ItemDropZone);
	}

	void FFilterViewModel_UserAssetTagCollection::RenameInternal(FName NewName)
	{
		Cast<UTaggedAssetBrowserFilter_UserAssetTagCollection>(Element)->SetCollectionName(NewName);
	}
}
#undef LOCTEXT_NAMESPACE
