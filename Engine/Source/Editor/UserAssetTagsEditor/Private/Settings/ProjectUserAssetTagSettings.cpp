// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectUserAssetTagSettings.h"

TSet<FName> UProjectUserAssetTagSettings::GetProjectUserAssetTagsForClass(const UClass* Class) const
{
	FSoftClassPath ClassPath(Class);
	if(UserAssetTagsPerType.Contains(ClassPath))
	{
		return UserAssetTagsPerType[ClassPath].FavoriteUserAssetTags;
	}

	return {};
}

FName UProjectUserAssetTagSettings::GetCategoryName() const
{
	return NAME_Editor;
}
