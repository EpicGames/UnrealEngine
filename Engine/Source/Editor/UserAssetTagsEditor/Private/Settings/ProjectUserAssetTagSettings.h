// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Config/LocalFavoriteUserAssetTagsConfig.h"
#include "Engine/DeveloperSettings.h"
#include "ProjectUserAssetTagSettings.generated.h"

/**
 * 
 */
UCLASS(Config=Editor, DefaultConfig, DisplayName="User Asset Tags", MinimalAPI)
class UProjectUserAssetTagSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Config, Category="UserAssetTags")
	TMap<FSoftClassPath, FPerTypeFavoriteUserAssetTags> UserAssetTagsPerType;

	TSet<FName> GetProjectUserAssetTagsForClass(const UClass* Class) const;

	virtual FName GetCategoryName() const override;
};
