// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaggedAssetBrowserEditorStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SegmentedControlStyle.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"

TSharedPtr<FTaggedAssetBrowserEditorStyle> FTaggedAssetBrowserEditorStyle::TaggedAssetBrowserEditorStyle = nullptr;

void FTaggedAssetBrowserEditorStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FTaggedAssetBrowserEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

void FTaggedAssetBrowserEditorStyle::Shutdown()
{
	Unregister();
	TaggedAssetBrowserEditorStyle.Reset();
}

FTaggedAssetBrowserEditorStyle::FTaggedAssetBrowserEditorStyle() : FSlateStyleSet("TaggedAssetBrowserEditorStyle")
{
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	InitStyle();
}

void FTaggedAssetBrowserEditorStyle::InitStyle()
{
	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");

	Set("TaggedAssetBrowser.PropertySeparator", new FSlateColorBrush(FStyleColors::White25));

	FTextBlockStyle AssetBrowserAssetTitleStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText.Important");
	AssetBrowserAssetTitleStyle.Font.Size = 12.f;
	Set("TaggedAssetBrowser.AssetTitle", AssetBrowserAssetTitleStyle);
	
	FTextBlockStyle AssetBrowserAssetTypeStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText.Subdued");
	AssetBrowserAssetTypeStyle.Font.Size = 10.f;
	Set("TaggedAssetBrowser.AssetType", AssetBrowserAssetTypeStyle);
	
	Set("TaggedAssetBrowser.AssetTag.OuterBorder", new FSlateRoundedBoxBrush(FStyleColors::AccentGray, 6.f));
	Set("TaggedAssetBrowser.AssetTag.InnerBorder", new FSlateRoundedBoxBrush(FStyleColors::Black, 4.f));

	FTextBlockStyle AssetTagText = NormalText;

	FSlateFontInfo Font = FStyleFonts::Get().Normal;
	Font.Size = 10.f;
	AssetTagText.SetFont(Font);
	Set("TaggedAssetBrowser.AssetTag.Text", AssetTagText);
}

void FTaggedAssetBrowserEditorStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const FTaggedAssetBrowserEditorStyle& FTaggedAssetBrowserEditorStyle::Get()
{
	if(!TaggedAssetBrowserEditorStyle.IsValid())
	{
		TaggedAssetBrowserEditorStyle = MakeShareable(new FTaggedAssetBrowserEditorStyle());
	}
	
	return *TaggedAssetBrowserEditorStyle;
}

void FTaggedAssetBrowserEditorStyle::ReinitializeStyle()
{
	Unregister();
	TaggedAssetBrowserEditorStyle.Reset();
	Register();	
}
