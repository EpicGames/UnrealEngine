// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaggedAssetBrowserFilters/TaggedAssetBrowser_CommonFilters.h"

#include "IContentBrowserDataModule.h"
#include "UserAssetTagEditorUtilities.h"
#include "UserAssetTagsEditorModule.h"
#include "Logging/StructuredLog.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/STaggedAssetBrowser.h"
#include "Engine/Texture2D.h"

#define LOCTEXT_NAMESPACE "UserAssetTags"

const FSlateBrush* FTaggedAssetBrowserSectionIconData::GetImageBrush() const
{
	if(bUseTextureForIcon)
	{
		if(TextureBrush.GetResourceObject() != Icon)
		{
			Icon->WaitForStreaming();
			Icon->UpdateResource();
			TextureBrush.SetResourceObject(Icon);
		}

		if(TextureBrush.GetResourceObject())
		{
			return &TextureBrush;
		}

		return FAppStyle::GetNoBrush();
	}

	// Seems "None" will return some brush that is not the NoBrush, so we do this manually
	if(StyleName.IsNone())
	{
		return FAppStyle::GetNoBrush();
	}
	
	if(const FSlateBrush* Brush = FAppStyle::GetOptionalBrush(StyleName))
	{
		return Brush;
	}
	
	return FAppStyle::GetNoBrush();
}

FString UTaggedAssetBrowserFilter_UserAssetTag::ToString() const
{
	return UserAssetTag.ToString();
}

FSlateIcon UTaggedAssetBrowserFilter_UserAssetTag::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Tag");
}

bool UTaggedAssetBrowserFilter_UserAssetTag::DoesAssetHaveTag(const FAssetData& AssetCandidate) const
{
	return UE::UserAssetTags::HasUserAssetTag(AssetCandidate, UserAssetTag);
}

void UTaggedAssetBrowserFilter_UserAssetTag::ModifyARFilterInternal(FARFilter& Filter) const
{
	// We query for assets with this specific tag. Since TagsAndValues works as an "OR", we move the parent-tag checking into ShouldFilterAsset instead.
	// Ideally, we could query the AR exactly how we need it, but as long as we can't do this, we first query for the main tag, and then filter the results
	Filter.TagsAndValues.Add(FName(UE::UserAssetTags::GetUATPrefixedTag(UserAssetTag)), TOptional<FString>());
}

bool UTaggedAssetBrowserFilter_UserAssetTag::ShouldFilterAssetInternal(const FAssetData& InAssetData) const
{
	// We test the current tag filter, and all parent tag filters up the hierarchy
	TArray<const UTaggedAssetBrowserFilter_UserAssetTag*> TagsToTest = { this };
	
	for(const UTaggedAssetBrowserFilter_UserAssetTag* Parent = this; Parent; Parent = Cast<UTaggedAssetBrowserFilter_UserAssetTag>(Parent->GetOuter()))
	{
		TagsToTest.Add(Parent);
	}

	for(const UTaggedAssetBrowserFilter_UserAssetTag* TagToTest : TagsToTest)
	{
		// For tag combinations, we have to make sure the asset has this current tag as well.
		// Multiple tag filters writing to the ARFilter will get assets containing EITHER of the tags
		// So if we combine a tag filter in the section with a tag filter in the hierarchy, we filter our the assets that don't contain all of the requested tags
		if(TagToTest->DoesAssetHaveTag(InAssetData) == false)
		{
			return true;
		}
	}
	
	return false;
}

void UTaggedAssetBrowserFilter_UserAssetTag::SetUserAssetTag(FName InUserAssetTag)
{
	UserAssetTag = InUserAssetTag;
}

FString UTaggedAssetBrowserFilter_UserAssetTagCollection::ToString() const
{
	return Name.ToString();
}

FText UTaggedAssetBrowserFilter_UserAssetTagCollection::GetTooltip() const
{
	return Description;
}

FSlateIcon UTaggedAssetBrowserFilter_UserAssetTagCollection::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.TagCollection");
}

void UTaggedAssetBrowserFilter_UserAssetTagCollection::CreateAdditionalWidgets(TSharedPtr<SHorizontalBox> ExtensionBox)
{
	ExtensionBox->AddSlot()
	.AutoWidth()
	.Padding(2.f)
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), "HintText")
		.Text_UObject(this, &UTaggedAssetBrowserFilter_UserAssetTagCollection::GetContainedChildrenNumberText)
		.Visibility_UObject(this, &UTaggedAssetBrowserFilter_UserAssetTagCollection::GetContainedChildrenNumberTextVisibility)
		.ToolTipText_UObject(this, &UTaggedAssetBrowserFilter_UserAssetTagCollection::GetContainedChildrenNumberTooltip)
	];
}

void UTaggedAssetBrowserFilter_UserAssetTagCollection::ModifyARFilterInternal(FARFilter& Filter) const
{
	TArray<const UTaggedAssetBrowserFilter_UserAssetTag*> UserAssetTagChildren;
	GetChildrenOfType(UserAssetTagChildren, false);

	// We let each user asset tag filter modify the ARFilter, which works as an 'OR' filter
	for(const UTaggedAssetBrowserFilter_UserAssetTag* UserAssetTagFilter : UserAssetTagChildren)
	{
		UserAssetTagFilter->ModifyARFilter(Filter);
	}
}

bool UTaggedAssetBrowserFilter_UserAssetTagCollection::ShouldFilterAssetInternal(const FAssetData& InAssetData) const
{
	TArray<const UTaggedAssetBrowserFilter_UserAssetTag*> UserAssetTagChildren;
	GetChildrenOfType(UserAssetTagChildren, false);

	bool bDoesAnyTagPass = false;

	for(const UTaggedAssetBrowserFilter_UserAssetTag* UserAssetTagFilter : UserAssetTagChildren)
	{
		if(UserAssetTagFilter->ShouldFilterAsset(InAssetData) == false)
		{
			bDoesAnyTagPass = true;
		}

		if(bDoesAnyTagPass)
		{
			return false;
		}
	}

	return true;
}

FText UTaggedAssetBrowserFilter_UserAssetTagCollection::GetContainedChildrenNumberText() const
{
	FText BaseMessage = FText::AsCultureInvariant("(+{0})");
	return FText::FormatOrdered(BaseMessage, FText::AsNumber(GetChildren().Num()));
}

EVisibility UTaggedAssetBrowserFilter_UserAssetTagCollection::GetContainedChildrenNumberTextVisibility() const
{
	return GetChildren().Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

FText UTaggedAssetBrowserFilter_UserAssetTagCollection::GetContainedChildrenNumberTooltip() const
{
	FText BaseMessage = LOCTEXT("UserAssetTagCollection_ChildrenHintTooltip", "This collection contains {0} {0}|plural(one=tag,other=tags).");
	return FText::FormatOrdered(BaseMessage, FText::AsNumber(GetChildren().Num()));
}

void UTaggedAssetBrowserFilter_UserAssetTagCollection::SetCollectionName(FName InName)
{
	Name = InName;
}

void UTaggedAssetBrowserFilter_Recent::InitializeInternal(const FTaggedAssetBrowserContext& InContext)
{
	ListAttribute = InContext.TaggedAssetBrowser.Pin()->GetFavoritesListAttribute();
	if(ListAttribute.Get(nullptr) == nullptr)
	{
		UE_LOGFMT(LogUserAssetTags, Warning, "Filter could not be initialized. No favorites list specified.");
	}
}

bool UTaggedAssetBrowserFilter_Recent::ShouldFilterAssetInternal(const FAssetData& InAssetData) const
{
	return !IsAssetRecent(InAssetData);
}

FSlateIcon UTaggedAssetBrowserFilter_Recent::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Recent");
}

bool UTaggedAssetBrowserFilter_Recent::IsAssetRecent(const FAssetData& AssetCandidate) const
{
	if(ListAttribute.Get(nullptr) != nullptr)
	{
		return ListAttribute.Get()->FindMRUItemIdx(AssetCandidate.PackageName.ToString()) != INDEX_NONE;
	}

	return false;
}

FString UTaggedAssetBrowserFilter_Directories::ToString() const
{
	if (!FilterName.IsNone())
	{
		return FilterName.ToString();
	}

	if (!DirectoryPaths.IsEmpty())
	{
		const FDirectoryPath& FirstDirectoryPath = DirectoryPaths[0];

		int32 LeafIndex = INDEX_NONE;
		if (FirstDirectoryPath.Path.FindLastChar(TEXT('/'), LeafIndex))
		{
			if (LeafIndex != INDEX_NONE)
			{
				return FirstDirectoryPath.Path.RightChop(LeafIndex + 1);
			}
		}
	}

	return UTaggedAssetBrowserFilterBase::ToString();
}

void UTaggedAssetBrowserFilter_Directories::ModifyARFilterInternal(FARFilter& Filter) const
{
	// The AR would use regular package paths, but in the context of the Content Browser/Asset Picker, which uses content sources and virtual paths
	// the PackagePaths are repurposed. We need to convert the regular PackagePaths into virtual ones first.
			
	UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = IContentBrowserDataModule::Get().GetSubsystem();
	TArray<FName> VirtualPackagePaths;
	Algo::Transform(DirectoryPaths, VirtualPackagePaths, [ContentBrowserDataSubsystem](const FDirectoryPath& DirectoryPath)
	{
		FName VirtualPath;
		ContentBrowserDataSubsystem->ConvertInternalPathToVirtual(DirectoryPath.Path, VirtualPath);
		return VirtualPath;
	});

	Filter.PackagePaths.Append(VirtualPackagePaths);
}

FText UTaggedAssetBrowserFilter_Directories::GetTooltip() const
{
	if (!DirectoryPaths.IsEmpty())
	{
		FText BaseTooltip = LOCTEXT("UTaggedAssetBrowserFilter_DirectoriesTooltip", "Directories: \n {0}.");
		FTextBuilder TooltipListBuilder;

		for (const FDirectoryPath& DirectoryPath : DirectoryPaths)
		{
			TooltipListBuilder.AppendLine(DirectoryPath.Path);
		}

		return FText::FormatOrdered(BaseTooltip, TooltipListBuilder.ToText());
	}

	return UTaggedAssetBrowserFilterBase::GetTooltip();
}

FSlateIcon UTaggedAssetBrowserFilter_Directories::GetIcon() const
{
	if(ActiveContext.IsSet())
	{
		if(IsSelectedFilter())
		{
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderOpen");
		}
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderClosed");
}

FString UTaggedAssetBrowserFilter_Class::ToString() const
{
	if (!FilterName.IsNone())
	{
		return FilterName.ToString();
	}

	if (!Classes.IsEmpty())
	{
		const UClass* FirstClass = Classes[0];
		if (FirstClass)
		{
			return FirstClass->GetDisplayNameText().ToString();
		}
	}

	return UTaggedAssetBrowserFilterBase::ToString();
}

FText UTaggedAssetBrowserFilter_Class::GetTooltip() const
{
	if (!Classes.IsEmpty())
	{
		FText BaseTooltip = LOCTEXT("UTaggedAssetBrowserFilter_ClassTooltip", "Classes: \n {0}.");
		FTextBuilder TooltipListBuilder;

		for (const UClass* Class : Classes)
		{
			if (Class)
			{
				TooltipListBuilder.AppendLine(Class->GetDisplayNameText().ToString());
			}
		}

		return FText::FormatOrdered(BaseTooltip, TooltipListBuilder.ToText());
	}

	return UTaggedAssetBrowserFilterBase::GetTooltip();
}

FSlateIcon UTaggedAssetBrowserFilter_Class::GetIcon() const
{
	if (!Classes.IsEmpty())
	{
		const UClass* FirstClass = Classes[0];
		if (FirstClass)
		{
			return FSlateIconFinder::FindIconForClass(FirstClass);
		}
	}

	return UTaggedAssetBrowserFilterBase::GetIcon();
}

void UTaggedAssetBrowserFilter_Class::ModifyARFilterInternal(FARFilter& Filter) const
{
	for(UClass* Class : Classes)
	{
		if(Class)
		{
			Filter.ClassPaths.Add(Class->GetClassPathName());
		}
	}
	Filter.bRecursiveClasses = bAllowChildClasses;
}

#undef LOCTEXT_NAMESPACE
