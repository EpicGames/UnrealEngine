// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaggedAssetBrowserMenuFilters.h"

#include "Assets/TaggedAssetBrowserConfiguration.h"

#define LOCTEXT_NAMESPACE "UserAssetTagMenuFilters"

FName UTaggedAssetBrowserFilterBase::GetIdentifier() const
{
	TOptional<FString> InstanceIdentifier = GetInstanceIdentifier();

	if(InstanceIdentifier.IsSet())
	{
		return FName(GetClass()->GetFName().ToString() + "_" + InstanceIdentifier.GetValue());
	}

	return GetClass()->GetFName();
}

bool UTaggedAssetBrowserFilterBase::IsSelectedFilter() const
{
	if(ensure(ActiveContext.IsSet()))
	{
		return ActiveContext->OnGetSelectedFilters.Execute().Contains(this);
	}

	return false;
}

void UTaggedAssetBrowserFilterBase::ModifyARFilter(FARFilter& Filter) const
{
	ModifyARFilterInternal(Filter);
}

bool UTaggedAssetBrowserFilterBase::ShouldFilterAsset(const FAssetData& InAssetData) const
{
	return ShouldFilterAssetInternal(InAssetData);
}

bool UTaggedAssetBrowserFilterBase::IsExtensionFilter() const
{
	if (UTaggedAssetBrowserConfiguration* OwningConfig = GetTypedOuter<UTaggedAssetBrowserConfiguration>())
	{
		return OwningConfig->bIsExtension;
	}

	return false;
}

void UTaggedAssetBrowserFilterBase::Initialize(const FTaggedAssetBrowserContext& InContext)
{
	ActiveContext = InContext;
	
	InitializeInternal(InContext);
}

FSlateIcon UTaggedAssetBrowserFilterBase::GetIcon() const
{
	return FSlateIcon();
}

const FSlateBrush* UTaggedAssetBrowserFilterBase::GetIconBrush() const
{
	return GetIcon().GetIcon();
}

bool UTaggedAssetBrowserFilterBase::DoesFilterMatchTextQuery(const FText& Text)
{
	if(ToString().Replace(TEXT(" "), TEXT("")).Contains(Text.ToString()))
	{
		return true;
	}

	return false;
}

uint32 GetTypeHash(const UTaggedAssetBrowserFilterBase& Filter)
{
	return GetTypeHash(Filter.GetIdentifier());
}

#undef LOCTEXT_NAMESPACE
