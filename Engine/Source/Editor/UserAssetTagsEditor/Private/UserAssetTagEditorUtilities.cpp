// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserAssetTagEditorUtilities.h"

#include "AssetToolsModule.h"
#include "DataHierarchyViewModelBase.h"
#include "LevelEditor.h"
#include "TaggedAssetBrowserMenuFilters.h"
#include "UserAssetTagsEditorModule.h"
#include "Misc/App.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Widgets/SUserAssetTagsEditor.h"
#include "Framework/Application/SlateApplication.h"
#include "Logging/StructuredLog.h"
#include "Widgets/SDataHierarchyEditor.h"

#define LOCTEXT_NAMESPACE "UserAssetTags"

TArray<FName> UE::UserAssetTags::GetUserAssetTagsForAssetData(const FAssetData& InAssetData)
{
	if(InAssetData.IsAssetLoaded())
	{
		UObject* Asset = InAssetData.GetAsset();
		return GetUserAssetTagsForObject(Asset);
	}

	TArray<FName> Result;
	
	for(const auto& Pair : InAssetData.TagsAndValues)
	{
		FNameBuilder Builder(Pair.Key);
		
		if(Builder.ToView().StartsWith(UAT_METADATA_PREFIX))
		{
			Builder.RemoveAt(0, UAT_METADATA_PREFIX.Len());
			Result.Add(FName(Builder));
		}
	}
	
	return Result;
}



TArray<FName> UE::UserAssetTags::GetUserAssetTagsForObject(const UObject* InObject, bool bWithPrefix)
{
	TArray<FName> Result;
	
	check(InObject);
	UPackage* Package = InObject->GetPackage();
	check(Package);
	FMetaData& Metadata = Package->GetMetaData();
	return GetUserAssetTagsFromMetaData(Metadata, bWithPrefix);
}

TArray<FName> UE::UserAssetTags::GetUserAssetTagsFromMetaData(const FMetaData& MetaData, bool bWithPrefix)
{
	TArray<FName> Result;
	
	for(const auto& Pair : MetaData.RootMetaDataMap)
	{
		FNameBuilder Builder(Pair.Key);
		
		if(Builder.ToView().StartsWith(UAT_METADATA_PREFIX))
		{
			if(bWithPrefix == false)
			{
				Builder.RemoveAt(0, UAT_METADATA_PREFIX.Len());
			}
			
			Result.Add(FName(Builder));
		}
	}

	return Result;
}

bool UE::UserAssetTags::HasUserAssetTag(const FAssetData& InAssetData, FName UserAssetTag)
{
	UserAssetTag = GetUATPrefixedTag(UserAssetTag);
	
	if(InAssetData.IsAssetLoaded())
	{
		return UE::UserAssetTags::HasUserAssetTag(InAssetData.GetAsset(), UserAssetTag);
	}

	return InAssetData.TagsAndValues.FindTag(UserAssetTag).IsSet();
}

bool UE::UserAssetTags::HasUserAssetTag(UObject* Object, FName UserAssetTag)
{
	UserAssetTag = GetUATPrefixedTag(UserAssetTag);

	UPackage* Package = Object->GetPackage();
	check(Package);
	const FMetaData& Metadata = Package->GetMetaData();
	
	return Metadata.RootMetaDataMap.Contains(UserAssetTag);
}

void UE::UserAssetTags::AddUserAssetTag(UObject* Object, FName UserAssetTag)
{
	UserAssetTag = GetUATPrefixedTag(UserAssetTag);
	
	if(HasUserAssetTag(Object, UserAssetTag) == false)
	{
		UPackage* Package = Object->GetPackage();
		check(Package);
		FMetaData& Metadata = Package->GetMetaData();
		Metadata.RootMetaDataMap.Add(UserAssetTag);
	}
}

bool UE::UserAssetTags::RemoveUserAssetTag(UObject* Object, FName UserAssetTag)
{
	UserAssetTag = GetUATPrefixedTag(UserAssetTag);

	UPackage* Package = Object->GetPackage();
	check(Package);
	FMetaData& Metadata = Package->GetMetaData();
	
	return Metadata.RootMetaDataMap.Remove(UserAssetTag) != 0;
}

FName UE::UserAssetTags::GetUATPrefixedTag(FName InUserAssetTag)
{
	FNameBuilder Builder(InUserAssetTag);
	if(Builder.ToView().StartsWith(UAT_METADATA_PREFIX) == false)
	{
		Builder.Prepend(UAT_METADATA_PREFIX);
		return Builder.ToString();
	}
	
	return InUserAssetTag;
}

FName UE::UserAssetTags::GetUATWithoutPrefix(FName InUserAssetTag)
{
	FNameBuilder Builder(InUserAssetTag);
	if(Builder.ToView().StartsWith(UAT_METADATA_PREFIX))
	{
		Builder.RemoveAt(0, UAT_METADATA_PREFIX.Len());
	}
	
	return InUserAssetTag;
}

void UE::UserAssetTags::SummonUserAssetTagsEditor()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	LevelEditorTabManager->TryInvokeTab(FUserAssetTagsEditorModule::ManageTagsTabId);
}

TArray<FAssetData> UE::UserAssetTags::FindStandaloneConfigurationForExtension(const UTaggedAssetBrowserConfiguration* InExtensionAsset)
{
	if(!ensure(InExtensionAsset && InExtensionAsset->bIsExtension == true))
	{
		return {};
	}
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		
	FARFilter ExtensionFilter;
	ExtensionFilter.ClassPaths.Add(UTaggedAssetBrowserConfiguration::StaticClass()->GetClassPathName());
	ExtensionFilter.TagsAndValues.Add(FName("bIsExtension"), FString("False"));
	ExtensionFilter.bIncludeOnlyOnDiskAssets = false;

	TArray<FAssetData> ExtensionAssets;
	AssetRegistryModule.Get().GetAssets(ExtensionFilter, ExtensionAssets);

	FARFilter ProfileNameFilter;
	ProfileNameFilter.ClassPaths.Add(UTaggedAssetBrowserConfiguration::StaticClass()->GetClassPathName());
	ProfileNameFilter.TagsAndValues.Add(FName("ProfileName"), InExtensionAsset->ProfileName.ToString());
	ProfileNameFilter.bIncludeOnlyOnDiskAssets = false;

	TArray<FAssetData> MatchingProfileNameAssets;
	AssetRegistryModule.Get().GetAssets(ProfileNameFilter, MatchingProfileNameAssets);

	TSet<FAssetData> ExtensionSet;
	ExtensionSet.Append(ExtensionAssets);

	TSet<FAssetData> MatchingProfileSet;
	MatchingProfileSet.Append(MatchingProfileNameAssets);

	TSet<FAssetData> Result = ExtensionSet.Intersect(MatchingProfileSet);
	
	return Result.Array();
}

TArray<FName> UE::UserAssetTags::FindAllStandaloneConfigurationAssetProfileNames()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		
	FARFilter StandaloneFilter;
	StandaloneFilter.ClassPaths.Add(UTaggedAssetBrowserConfiguration::StaticClass()->GetClassPathName());
	StandaloneFilter.TagsAndValues.Add(GET_MEMBER_NAME_CHECKED(UTaggedAssetBrowserConfiguration, bIsExtension), FString("False"));
	StandaloneFilter.bIncludeOnlyOnDiskAssets = false;

	TArray<FAssetData> AssetData;
	AssetRegistryModule.Get().GetAssets(StandaloneFilter, AssetData);

	TArray<FName> Result;
	Algo::TransformIf(AssetData, Result, [](const FAssetData& AssetData)
	{
		return AssetData.FindTag(GET_MEMBER_NAME_CHECKED(UTaggedAssetBrowserConfiguration, ProfileName));
	},
	[](const FAssetData& AssetData)
	{
		return AssetData.GetTagValueRef<FName>(GET_MEMBER_NAME_CHECKED(UTaggedAssetBrowserConfiguration, ProfileName));
	});
	
	return Result;
}

#undef LOCTEXT_NAMESPACE
