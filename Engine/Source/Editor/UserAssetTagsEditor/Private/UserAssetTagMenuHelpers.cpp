// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserAssetTagMenuHelpers.h"

#include "ContentBrowserMenuContexts.h"
#include "TaggedAssetBrowserMenuFilters.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "TaggedAssetBrowserMenuFilters.h"
#include "ToolMenus.h"
#include "UserAssetTagEditorUtilities.h"
#include "UserAssetTagsEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Commands/UserAssetTagCommands.h"
#include "Config/TaggedAssetBrowserConfig.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "UserAssetTagMenuHelpers"

void UE::UserAssetTags::Menus::RegisterMenusAndProfiles()
{
	UToolMenus::Get()->RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
	{
		RegisterDefaultTaggedAssetBrowserViewOptionsProfile();
		ExtendContentBrowserAssetContextMenu();
	}));
}

void UE::UserAssetTags::Menus::RegisterDefaultTaggedAssetBrowserViewOptionsProfile()
{
	UToolMenus::Get()->RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
	{
		const FName MenuName("ContentBrowser.AssetViewOptions");
		const FName ProfileName("TaggedAssetBrowser");

		FToolMenuProfile* NiagaraAssetBrowserProfile = UToolMenus::Get()->AddRuntimeMenuProfile(MenuName, ProfileName);
		if(FCustomizedToolMenuSection* Section = NiagaraAssetBrowserProfile->AddSection("View"))
		{
			Section->Visibility = ECustomizedToolMenuVisibility::Hidden;
		}

		UToolMenu* AssetPickerAssetViewOptionsMenu = UToolMenus::Get()->ExtendMenu(MenuName);
		AssetPickerAssetViewOptionsMenu->AddDynamicSection("TaggedAssetBrowserDynamicSection", FNewSectionConstructChoice(FNewToolMenuDelegate::CreateLambda([](UToolMenu* ToolMenu)
		{
			// We only want to modify the menu if we are in a TaggedAssetBrowser profile
			if(UToolMenuProfileContext* ToolMenuProfileContext = ToolMenu->FindContext<UToolMenuProfileContext>())
			{
				if(ToolMenuProfileContext->ActiveProfiles.Contains("TaggedAssetBrowser") == false)
				{
					return;
				}
			}
			else
			{
				return;
			}

			FToolMenuSection& TaggedAssetBrowserSection = ToolMenu->AddSection("TaggedAssetBrowser", LOCTEXT("TaggedAssetBrowser", "Tagged Asset Browser"));
			
			// Hidden
			{
				auto GetShowHiddenAssetsCheckState = []() -> ECheckBoxState
			   {
				   return UTaggedAssetBrowserConfig::Get()->bShowHiddenAssets ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			   };
			
			   auto OnShowHiddenAssetsToggled = []() -> void
			   {
				   UTaggedAssetBrowserConfig::Get()->bShowHiddenAssets = !UTaggedAssetBrowserConfig::Get()->bShowHiddenAssets;
				   UTaggedAssetBrowserConfig::Get()->PostEditChange();
			   };
			
			   FToolUIAction Action;
			   Action.ExecuteAction = FToolMenuExecuteAction::CreateLambda([&GetShowHiddenAssetsCheckState, &OnShowHiddenAssetsToggled](const FToolMenuContext& Context)
			   {
				   OnShowHiddenAssetsToggled();
			   });
			   Action.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([&GetShowHiddenAssetsCheckState](const FToolMenuContext& Context)
			   {
				   return GetShowHiddenAssetsCheckState();
			   });

			   FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
				   FName("NiagaraShowHidden"),
				   LOCTEXT("ShowHiddenLabel", "Show Hidden"),
				   LOCTEXT("ShowHiddenTooltip", "Show assets that were tagged as Hidden"),
				   FSlateIcon(),
				   FToolUIActionChoice(Action), EUserInterfaceActionType::ToggleButton);
				
			   TaggedAssetBrowserSection.AddEntry(Entry);
			}
			
			// Deprecation
			{
				auto GetShowDeprecatedAssetsCheckState = []() -> ECheckBoxState
			   {
				   return UTaggedAssetBrowserConfig::Get()->bShowDeprecatedAssets ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			   };
			
			   auto OnShowDeprecatedAssetsToggled = []() -> void
			   {
				   UTaggedAssetBrowserConfig::Get()->bShowDeprecatedAssets = !UTaggedAssetBrowserConfig::Get()->bShowDeprecatedAssets;
				   UTaggedAssetBrowserConfig::Get()->PostEditChange();
			   };
			
			   FToolUIAction Action;
			   Action.ExecuteAction = FToolMenuExecuteAction::CreateLambda([&GetShowDeprecatedAssetsCheckState, &OnShowDeprecatedAssetsToggled](const FToolMenuContext& Context)
			   {
				   OnShowDeprecatedAssetsToggled();
			   });
			   Action.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([&GetShowDeprecatedAssetsCheckState](const FToolMenuContext& Context)
			   {
				   return GetShowDeprecatedAssetsCheckState();
			   });

			   FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
				   FName("ShowDeprecated"),
				   LOCTEXT("ShowDeprecatedLabel", "Show Deprecated"),
				   LOCTEXT("ShowDeprecatedTooltip", "Show assets that were tagged as Deprecated"),
				   FSlateIcon(),
				   FToolUIActionChoice(Action), EUserInterfaceActionType::ToggleButton);
				
				TaggedAssetBrowserSection.AddEntry(Entry);
			}
		})));
	}));
}

void UE::UserAssetTags::Menus::ExtendContentBrowserAssetContextMenu()
{
	using namespace UE::UserAssetTags;

	FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
	
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu");

	// The command list which contains functionality is appended to the content browser via listener in FUserAssetTagsEditorModule::OnRegisterCommandList
	FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(FUserAssetTagCommands::Get().ManageTags, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Tag"));
	Menu->AddMenuEntry("CommonAssetActions", Entry);
}

UE::UserAssetTags::Menus::FMultipleAssetsTagInfo UE::UserAssetTags::Menus::GatherInfoAboutTags(const TArray<FAssetData>& Assets)
{
	FMultipleAssetsTagInfo Result;

	for(const FAssetData& AssetData : Assets)
	{
		for(const FName& Tag : UE::UserAssetTags::GetUserAssetTagsForAssetData(AssetData))
		{
			Result.TagInfos.FindOrAdd(Tag).AssetsOwningThisTag.Add(AssetData);
		}
	}

	return Result;
}

TOptional<FText> UE::UserAssetTags::Menus::DetermineInfoText(const FMultipleAssetsTagSingleTagInfo& SingleTagInfo,
	const FMultipleAssetsTagInfo& TotalInfo)
{
	if(SingleTagInfo.AssetsOwningThisTag.Num() != TotalInfo.TagInfos.Num())
	{
		return FText::FromString("Only partial");
	}

	return TOptional<FText>();
}

void UE::UserAssetTags::Menus::AddTagsToAssets(FName InUserAssetTag, const TArray<FAssetData>& InAssetData)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("Transaction_AddedUserAssetTags", "Added User Asset Tags"));
	bool bShouldTransact = false;
	
	for(FAssetData AssetData : InAssetData)
	{
		UObject* Asset = AssetData.GetAsset();

		if(UE::UserAssetTags::HasUserAssetTag(Asset, InUserAssetTag) == false)
		{
			bShouldTransact = true;
			
			Asset->GetPackage()->SetFlags(RF_Transactional);
			Asset->GetPackage()->Modify();
			UE::UserAssetTags::AddUserAssetTag(Asset, InUserAssetTag);
		}
	}

	if(bShouldTransact == false)
	{
		ScopedTransaction.Cancel();
	}
}

void UE::UserAssetTags::Menus::RemoveTagsFromAssets(FName InUserAssetTag, const TArray<FAssetData>& InAssetData)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("Transaction_RemovedUserAssetTags", "Removed User Asset Tags"));
	bool bShouldTransact = false;

	for(FAssetData AssetData : InAssetData)
	{
		UObject* Asset = AssetData.GetAsset();

		if(UE::UserAssetTags::HasUserAssetTag(Asset, InUserAssetTag))
		{
			bShouldTransact = true;

			Asset->GetPackage()->SetFlags(RF_Transactional);
			Asset->GetPackage()->Modify();
			UE::UserAssetTags::RemoveUserAssetTag(Asset, InUserAssetTag);
		}
	}

	if(bShouldTransact == false)
	{
		ScopedTransaction.Cancel();
	}
}

ECheckBoxState UE::UserAssetTags::Menus::DetermineTagUIStatus(FName InUserAssetTag, const TArray<FAssetData>& InAssetData)
{
	int32 NumAssetsOwningTag = 0;
	for(FAssetData AssetData : InAssetData)
	{
		if(UE::UserAssetTags::HasUserAssetTag(AssetData, InUserAssetTag))
		{
			NumAssetsOwningTag++;
		}
	}

	if(NumAssetsOwningTag == 0)
	{
		return ECheckBoxState::Unchecked;
	}

	if(NumAssetsOwningTag == InAssetData.Num())
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Undetermined;
}

#undef LOCTEXT_NAMESPACE
