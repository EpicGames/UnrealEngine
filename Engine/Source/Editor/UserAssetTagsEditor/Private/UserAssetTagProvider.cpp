// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserAssetTagProvider.h"

FText UUserAssetTagProvider::GetDisplayNameText(const UUserAssetTagEditorContext* Context) const
{
	return GetClass()->GetDisplayNameText();
}

FText UUserAssetTagProvider::GetToolTipText(const UUserAssetTagEditorContext* Context) const
{
	return GetClass()->GetToolTipText();
}
