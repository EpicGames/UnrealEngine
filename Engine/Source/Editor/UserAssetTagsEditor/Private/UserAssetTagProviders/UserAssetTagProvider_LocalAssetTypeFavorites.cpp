// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserAssetTagProvider_LocalAssetTypeFavorites.h"

#include "UserAssetTagEditorMenuContexts.h"
#include "Config/LocalFavoriteUserAssetTagsConfig.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SUserAssetTagsEditor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "UserAssetTags"

UUserAssetTagProvider_LocalAssetTypeFavorites::UUserAssetTagProvider_LocalAssetTypeFavorites()
{
}

FText UUserAssetTagProvider_LocalAssetTypeFavorites::GetDisplayNameText(const UUserAssetTagEditorContext* Context) const
{
	return LOCTEXT("FavoritesDisplayName", "Local Asset Type Favorites");
}

TSet<FName> UUserAssetTagProvider_LocalAssetTypeFavorites::GetSuggestedUserAssetTags(const UUserAssetTagEditorContext* Context) const
{
	TSharedPtr<SUserAssetTagsEditor> UserAssetTagsEditor = Context->UserAssetTagsEditor.Pin();
	if(UserAssetTagsEditor->GetSelectedAssets().Num() > 0)
	{
		// Due to the IsValid check we know there is only one class, so we fetch it from the first selected asset
		const FAssetData& SelectedAsset = UserAssetTagsEditor->GetSelectedAssets()[0];
		if(ULocalFavoriteUserAssetTagsConfig::Get()->GetFavoriteUserAssetTags().Contains(FSoftClassPath(SelectedAsset.GetClass())))
		{
			return ULocalFavoriteUserAssetTagsConfig::Get()->GetFavoriteUserAssetTags()[FSoftClassPath(SelectedAsset.GetClass())].FavoriteUserAssetTags;
		}
	}

	return {};
}

UUserAssetTagProvider::FResultWithUserFeedback UUserAssetTagProvider_LocalAssetTypeFavorites::IsValid(const UUserAssetTagEditorContext* Context) const
{
	TSet<UClass*> Classes;

	Algo::Transform(Context->UserAssetTagsEditor.Pin()->GetSelectedAssets(), Classes, [](const FAssetData& AssetData)
	{
		return AssetData.GetClass();
	});

	FResultWithUserFeedback Result(Classes.Num() == 1);
	if(Result == false)
	{
		Result.UserFeedback = LOCTEXT("LocalFavoriteProviderInvalid_MultipleTypes", "This provider is only valid when there is a single type of asset in the selection");
	}

	return Result;
}

TSharedPtr<SWidget> UUserAssetTagProvider_LocalAssetTypeFavorites::AddAdditionalSuggestedWidgets(FName UserAssetTag, const UUserAssetTagEditorContext* Context) const
{
	return SNew(SButton)
		.ToolTipText_Lambda([UserAssetTag, Context]()
		{
			if(Context->UserAssetTagsEditor.Pin()->GetSelectedAssets().Num() > 0)
			{
				UClass* Class = Context->UserAssetTagsEditor.Pin()->GetSelectedAssets()[0].GetClass();
				TSet<FName> FavoriteUserAssetTags = ULocalFavoriteUserAssetTagsConfig::Get()->GetFavoriteUserAssetTagsForClass(Class);
				if(FavoriteUserAssetTags.Contains(UserAssetTag))
				{
					return FText::FormatOrdered(LOCTEXT("LocalAssetTypeFavorites_RemoveFromFavoritesTooltip", "Remove from {0} favorites"), Class->GetDisplayNameText());
				}
				else
				{
					return FText::FormatOrdered(LOCTEXT("LocalAssetTypeFavorites_AddToFavoritesTooltip", "Add to {0} favorites"), Class->GetDisplayNameText());
				}
			}
			
			return FText::GetEmpty();
		})
		.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
		.OnClicked_Lambda([UserAssetTag, Context]()
		{
			if(Context->UserAssetTagsEditor.Pin()->GetSelectedAssets().Num() > 0)
			{
				UClass* Class = Context->UserAssetTagsEditor.Pin()->GetSelectedAssets()[0].GetClass();
				ULocalFavoriteUserAssetTagsConfig::Get()->ToggleFavoriteUserAssetTag(Class, UserAssetTag);
				Context->UserAssetTagsEditor.Pin()->RefreshDataAndMenus();
			}
			
			return FReply::Handled();
		})
		[
			SNew(SBox)
			.WidthOverride(16.f)
			.HeightOverride(16.f)
			[
				SNew(SImage)
				.Image_Lambda([UserAssetTag, Context]()
				{
					if(Context->UserAssetTagsEditor.Pin()->GetSelectedAssets().Num() > 0)
					{
						UClass* Class = Context->UserAssetTagsEditor.Pin()->GetSelectedAssets()[0].GetClass();
						TSet<FName> FavoriteUserAssetTags = ULocalFavoriteUserAssetTagsConfig::Get()->GetFavoriteUserAssetTagsForClass(Class);
						if(FavoriteUserAssetTags.Contains(UserAssetTag))
						{
							return FAppStyle::GetBrush("Icons.Star");	
						}
						else
						{
							return FAppStyle::GetBrush("Icons.Star.Outline");
						}
					}

					return FAppStyle::GetNoBrush();
				})
			]
		];
}

TSharedPtr<SWidget> UUserAssetTagProvider_LocalAssetTypeFavorites::AddAdditionalOwnedTagWidgets(FName UserAssetTag, const UUserAssetTagEditorContext* Context) const
{
	return AddAdditionalSuggestedWidgets(UserAssetTag, Context);
}

#undef LOCTEXT_NAMESPACE
