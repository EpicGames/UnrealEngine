// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UserAssetTagProvider.h"
#include "UserAssetTagProvider_LocalAssetTypeFavorites.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

class UUserAssetTagEditorContext;

/**
 * The local favorites provider lets you add your own selection of favorite tags per asset type
 * for quick reuse without affecting project settings.
 */
UCLASS(MinimalAPI, DisplayName="Favorites by type")
class UUserAssetTagProvider_LocalAssetTypeFavorites : public UUserAssetTagProvider
{
	GENERATED_BODY()

public:
	UE_API UUserAssetTagProvider_LocalAssetTypeFavorites();

	UE_API virtual FText GetDisplayNameText(const UUserAssetTagEditorContext* Context) const override;
	UE_API virtual TSet<FName> GetSuggestedUserAssetTags(const UUserAssetTagEditorContext* Context) const override;

	/** Favorites are per asset type, so we disable this provider if we encounter multiple asset types. */
	UE_API virtual FResultWithUserFeedback IsValid(const UUserAssetTagEditorContext* Context) const override;

	UE_API virtual TSharedPtr<SWidget> AddAdditionalSuggestedWidgets(FName UserAssetTag, const UUserAssetTagEditorContext* Context) const override;
	UE_API virtual TSharedPtr<SWidget> AddAdditionalOwnedTagWidgets(FName UserAssetTag, const UUserAssetTagEditorContext* Context) const override;
};

#undef UE_API
