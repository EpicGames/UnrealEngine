// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserAssetTagProvider_Project.h"

#include "ISettingsModule.h"
#include "TaggedAssetBrowserEditorStyle.h"
#include "Settings/ProjectUserAssetTagSettings.h"
#include "Widgets/SUserAssetTagsEditor.h"
#include "Modules/ModuleManager.h"
#include "ToolMenu.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "UserAssetTags"

FText UUserAssetTagProvider_Project::GetDisplayNameText(const UUserAssetTagEditorContext* Context) const
{
	TSharedPtr<SUserAssetTagsEditor> UserAssetTagsEditor = Context->UserAssetTagsEditor.Pin();
	if(UserAssetTagsEditor->GetSelectedAssets().Num() > 0)
	{
		const FAssetData& SelectedAsset = UserAssetTagsEditor->GetSelectedAssets()[0];
		return FText::FormatOrdered(INVTEXT("{0} {1}"), SelectedAsset.GetClass()->GetDisplayNameText(), LOCTEXT("ProjectProviderDisplayName", "Project Tags"));
	}

	return Super::GetDisplayNameText(Context);
}

TSet<FName> UUserAssetTagProvider_Project::GetSuggestedUserAssetTags(const UUserAssetTagEditorContext* Context) const
{
	TSharedPtr<SUserAssetTagsEditor> UserAssetTagsEditor = Context->UserAssetTagsEditor.Pin();
	if(UserAssetTagsEditor->GetSelectedAssets().Num() > 0)
	{
		// Due to the IsValid check we know there is only one class, so we fetch it from the first selected asset
		const FAssetData& SelectedAsset = UserAssetTagsEditor->GetSelectedAssets()[0];
		return GetDefault<UProjectUserAssetTagSettings>()->GetProjectUserAssetTagsForClass(SelectedAsset.GetClass());
	}

	return {};
}

UUserAssetTagProvider::FResultWithUserFeedback UUserAssetTagProvider_Project::IsValid(const UUserAssetTagEditorContext* Context) const
{
	TSet<UClass*> Classes;

	Algo::Transform(Context->UserAssetTagsEditor.Pin()->GetSelectedAssets(), Classes, [](const FAssetData& AssetData)
	{
		return AssetData.GetClass();
	});

	FResultWithUserFeedback Result(Classes.Num() == 1);
	if(Result == false)
	{
		Result.UserFeedback = LOCTEXT("ProjectTagProviderInvalid_MultipleTypes", "This provider is only valid when there is a single type of asset in the selection");
	}

	return Result;
}

void UUserAssetTagProvider_Project::NavigateToProjectTags()
{
	if(ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		const UProjectUserAssetTagSettings* Settings = GetDefault<UProjectUserAssetTagSettings>();
		SettingsModule->ShowViewer(Settings->GetContainerName(), Settings->GetCategoryName(), Settings->GetSectionName());
	}
}

void UUserAssetTagProvider_Project::AddToolbarMenuEntries(UToolMenu* DynamicMenu, const UUserAssetTagEditorContext* Context) const
{
	TSharedPtr<SButton> Button = SNew(SButton)
	.ButtonStyle(FAppStyle::Get(), "SimpleButton")
	.ToolTipText(LOCTEXT("NavigateToProjectTagsTooltip", "Summons the Project Tags window to edit tags per type."))
	.OnClicked_Lambda([this]()
	{
		NavigateToProjectTags();
		return FReply::Handled();
	})
	[
		SNew(SImage)
		.Image(FAppStyle::Get().GetBrush("ProjectSettings.TabIcon"))
	];
	
	FToolMenuEntry MenuEntry = FToolMenuEntry::InitWidget("NavigateToProjectTags", Button.ToSharedRef(), LOCTEXT("ProjectTagsButtonLabel", "Project Tags"), true);
	DynamicMenu->AddMenuEntry(NAME_None, MenuEntry);
}

#undef LOCTEXT_NAMESPACE
