// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UserAssetTagProvider.h"
#include "UserAssetTagProvider_Project.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

/** 
 *  A provider with the commonly used tags for this project, per asset type.
 *  You can assign tags per asset type in the project settings.
 */
UCLASS(MinimalAPI, DisplayName="Project Tags by type")
class UUserAssetTagProvider_Project : public UUserAssetTagProvider
{
	GENERATED_BODY()

	UE_API virtual FText GetDisplayNameText(const UUserAssetTagEditorContext* Context) const override;
	UE_API virtual TSet<FName> GetSuggestedUserAssetTags(const UUserAssetTagEditorContext* Context) const override;
	UE_API virtual FResultWithUserFeedback IsValid(const UUserAssetTagEditorContext* Context) const override;
	
	static UE_API void NavigateToProjectTags();
protected:
	UE_API virtual void AddToolbarMenuEntries(class UToolMenu* DynamicMenu, const UUserAssetTagEditorContext* Context) const override;
};

#undef UE_API
