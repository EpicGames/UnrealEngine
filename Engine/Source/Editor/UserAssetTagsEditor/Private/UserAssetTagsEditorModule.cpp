// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserAssetTagsEditorModule.h"
#include "TaggedAssetBrowserEditorStyle.h"
#include "UserAssetTagMenuHelpers.h"
#include "Editor.h"
#include "LevelEditor.h"
#include "UserAssetTagEditorUtilities.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Commands/UserAssetTagCommands.h"
#include "Config/LocalFavoriteUserAssetTagsConfig.h"
#include "Config/TaggedAssetBrowserConfig.h"
#include "Config/UserAssetTagsEditorConfig.h"
#include "Customizations/TaggedAssetBrowserConfigurationCustomization.h"
#include "DataHierarchyEditor/TaggedAssetBrowserConfigurationHierarchyViewModel.h"
#include "TaggedAssetBrowserFilters/TaggedAssetBrowser_CommonFilters.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Widgets/SUserAssetTagsEditor.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UserAssetTags"

DEFINE_LOG_CATEGORY(LogUserAssetTags);

FName FUserAssetTagsEditorModule::ManageTagsTabId = "ManageTagsTabId";

FUserAssetTagsEditorModule::FUserAssetTagsEditorModule()
{
}

FUserAssetTagsEditorModule::~FUserAssetTagsEditorModule()
{
}

void FUserAssetTagsEditorModule::StartupModule()
{
	CommandList = MakeShared<FUICommandList>();

	FTaggedAssetBrowserEditorStyle::Register();
	FUserAssetTagCommands::Register();

	// This registers multiple menu related settings, such as the Manage Tags menu, and the TAB View Profile.
	UE::UserAssetTags::Menus::RegisterMenusAndProfiles();

	RegisterDefaultConfigurationHierarchyElementViewModels();
	RegisterDetailsCustomizations();
	CommandList->MapAction(FUserAssetTagCommands::Get().ManageTags, FExecuteAction::CreateStatic(&UE::UserAssetTags::SummonUserAssetTagsEditor));

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnRegisterTabs().AddLambda([](TSharedPtr<FTabManager> TabManager)
	{
		FTabSpawnerEntry& TabSpawnerEntry = TabManager->RegisterTabSpawner(ManageTagsTabId, FOnSpawnTab::CreateStatic(&FUserAssetTagsEditorModule::SpawnUserAssetTagsEditorNomadTab));
		TabSpawnerEntry.SetMenuType(ETabSpawnerMenuType::Hidden);
		TabSpawnerEntry.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Tag"));
	});
	
	FTabManager::RegisterDefaultTabWindowSize(ManageTagsTabId, FVector2D(600.f, 650.f));
	
	// We have to call shutdown before the exit actually happens to be able to save to disk
	FEditorDelegates::OnEditorPreExit.AddLambda([]()
	{
		UUserAssetTagsEditorConfig::Shutdown();
		ULocalFavoriteUserAssetTagsConfig::Shutdown();
		UTaggedAssetBrowserConfig::Shutdown();
	});

	FInputBindingManager::Get().OnRegisterCommandList.AddRaw(this, &FUserAssetTagsEditorModule::OnRegisterCommandList);

	GetAssetRegistryTagsFromUserAssetTagsDelegateHandle = UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.AddStatic(&GetAssetRegistryTagsFromUserAssetTags);
	TransferUserAssetTagsFromOldPackageToNewPackageDelegateHandle = UPackage::OnMetaDataTransferRequestedDelegate.AddStatic(&TransferUserAssetTagsFromOldPackageToNewPackage);
}

void FUserAssetTagsEditorModule::ShutdownModule()
{
	UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.Remove(GetAssetRegistryTagsFromUserAssetTagsDelegateHandle);
	UPackage::OnMetaDataTransferRequestedDelegate.Remove(TransferUserAssetTagsFromOldPackageToNewPackageDelegateHandle);
	
	FInputBindingManager::Get().OnRegisterCommandList.RemoveAll(this);
	CommandList.Reset();
	FUserAssetTagCommands::Unregister();

	FTaggedAssetBrowserEditorStyle::Unregister();

	if(FLevelEditorModule* LevelEditorModule = FModuleManager::Get().GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditorModule->OnRegisterTabs().RemoveAll(this);

		if (TSharedPtr<FTabManager> TabManager = LevelEditorModule->GetLevelEditorTabManager())
		{
			TabManager->UnregisterTabSpawner(ManageTagsTabId);
		}
	}
	
	ConfigurationHierarchyViewModelFactories.Empty();
}

FUserAssetTagsEditorModule& FUserAssetTagsEditorModule::Get()
{
	return FModuleManager::LoadModuleChecked<FUserAssetTagsEditorModule>("UserAssetTagsEditor");
}

void FUserAssetTagsEditorModule::RegisterConfigurationHierarchyElementViewModel(TSubclassOf<UHierarchyElement> FilterType, FOnGetViewModelFactory InFactory)
{
	ensureMsgf(ConfigurationHierarchyViewModelFactories.Contains(FilterType) == false, TEXT("Tried adding a factory for the same type twice!"));
	ConfigurationHierarchyViewModelFactories.Add(FilterType, InFactory);
}

void FUserAssetTagsEditorModule::UnregisterConfigurationHierarchyElementViewModel(TSubclassOf<UHierarchyElement> FilterType)
{
	if(ensureMsgf(ConfigurationHierarchyViewModelFactories.Contains(FilterType) == true, TEXT("Tried deleting a factory despite not being registered!!")))
	{
		ConfigurationHierarchyViewModelFactories.Remove(FilterType);	
	}
}

void FUserAssetTagsEditorModule::RegisterDefaultConfigurationHierarchyElementViewModels()
{
	RegisterConfigurationHierarchyElementViewModel(UTaggedAssetBrowserFilter_All::StaticClass(), FOnGetViewModelFactory::CreateLambda([](UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InViewModel)
	{
		return MakeShared<UE::UserAssetTags::ViewModels::FFilterViewModel_All>(Cast<UTaggedAssetBrowserFilter_All>(Element), InParent.ToSharedRef(), InViewModel);
	}));

	RegisterConfigurationHierarchyElementViewModel(UTaggedAssetBrowserSection::StaticClass(), FOnGetViewModelFactory::CreateLambda([](UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InViewModel)
	{
		return MakeShared<UE::UserAssetTags::ViewModels::FFilterViewModel_Section>(Cast<UTaggedAssetBrowserSection>(Element), StaticCastSharedRef<FHierarchyRootViewModel>(InParent.ToSharedRef()), InViewModel);
	}));
	
	RegisterConfigurationHierarchyElementViewModel(UTaggedAssetBrowserFilter_Recent::StaticClass(), FOnGetViewModelFactory::CreateLambda([](UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InViewModel)
	{
		return MakeShared<FHierarchyItemViewModel>(Cast<UTaggedAssetBrowserFilter_Recent>(Element), InParent.ToSharedRef(), InViewModel);
	}));
	
	RegisterConfigurationHierarchyElementViewModel(UTaggedAssetBrowserFilter_UserAssetTag::StaticClass(), FOnGetViewModelFactory::CreateLambda([](UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InViewModel)
	{
		return MakeShared<UE::UserAssetTags::ViewModels::FFilterViewModel_UserAssetTag>(Cast<UTaggedAssetBrowserFilter_UserAssetTag>(Element), InParent.ToSharedRef(), InViewModel);
	}));

	RegisterConfigurationHierarchyElementViewModel(UTaggedAssetBrowserFilter_UserAssetTagCollection::StaticClass(), FOnGetViewModelFactory::CreateLambda([](UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InViewModel)
	{
		return MakeShared<UE::UserAssetTags::ViewModels::FFilterViewModel_UserAssetTagCollection>(Cast<UTaggedAssetBrowserFilter_UserAssetTagCollection>(Element), InParent.ToSharedRef(), InViewModel);
	}));

	RegisterConfigurationHierarchyElementViewModel(UTaggedAssetBrowserFilterRoot::StaticClass(), FOnGetViewModelFactory::CreateLambda([](UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InViewModel)
	{
		return MakeShared<UE::UserAssetTags::ViewModels::FFilterViewModel_Root>(Cast<UTaggedAssetBrowserFilterRoot>(Element), InViewModel, true);
	}));
}

void FUserAssetTagsEditorModule::RegisterDetailsCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	PropertyEditorModule.RegisterCustomClassLayout(
			UTaggedAssetBrowserConfiguration::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FTaggedAssetBrowserConfigurationCustomization::MakeInstance));
}

void FUserAssetTagsEditorModule::OnRegisterCommandList(FName ContextName, TSharedRef<FUICommandList> InCommandList)
{
	if(ContextName != "ContentBrowser")
	{
		return;
	}

	InCommandList->Append(CommandList.ToSharedRef());
}

TSharedRef<SDockTab> FUserAssetTagsEditorModule::SpawnUserAssetTagsEditorNomadTab(const FSpawnTabArgs& SpawnTabArgs)
{
	check(SpawnTabArgs.GetTabId() == ManageTagsTabId);

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
	.Label(LOCTEXT("ManageTagsTabLabel", "Manage Tags"))
	[
		SNew(SUserAssetTagsEditor)
	];
	
	return DockTab; 
}

void FUserAssetTagsEditorModule::GetAssetRegistryTagsFromUserAssetTags(FAssetRegistryTagsContext AssetRegistryTagsContext)
{
	const UObject* Object = AssetRegistryTagsContext.GetObject();
	if ((Object != nullptr) && (Object->GetClass() != nullptr) && Object->IsAsset())
	{
		// We add the tag with the UAT prefix as key so we can make use of AR key-caching and still identify UATs in the AR. Empty value with two spaces to bypass content requirements.
		for(const FName& UserAssetTag : UE::UserAssetTags::GetUserAssetTagsForObject(Object, true))
		{
			// Examples of tags: UAT.Fire, UAT.Template
			AssetRegistryTagsContext.AddTag(UObject::FAssetRegistryTag(UserAssetTag, TEXT("  "), UObject::FAssetRegistryTag::TT_Alphabetical));
		}
	}
}

void FUserAssetTagsEditorModule::TransferUserAssetTagsFromOldPackageToNewPackage(const FMetaData& OldMetaData, FMetaData& NewMetaData, const UPackage* OldPackage, UPackage* NewPackage)
{
	TArray<FName> UserAssetTags = UE::UserAssetTags::GetUserAssetTagsFromMetaData(OldMetaData, true);
	NewMetaData.RootMetaDataMap.Reserve(NewMetaData.RootMetaDataMap.Num() + UserAssetTags.Num());

	for(const FName& UserAssetTag : UserAssetTags)
	{
		NewMetaData.RootMetaDataMap.Add(UserAssetTag);
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FUserAssetTagsEditorModule, UserAssetTagsEditor)
