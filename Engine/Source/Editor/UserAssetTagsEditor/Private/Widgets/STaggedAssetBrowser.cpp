// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STaggedAssetBrowser.h"

#include "SAssetPicker.h"
#include "UserAssetTagEditorUtilities.h"
#include "UserAssetTagsEditorModule.h"
#include "ContentBrowserMenuContexts.h"
#include "IContentBrowserDataModule.h"
#include "SAssetView.h"
#include "SlateOptMacros.h"
#include "UserAssetTagEditorMenuContexts.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Logging/StructuredLog.h"
#include "Styling/ToolBarStyle.h"
#include "Config/TaggedAssetBrowserConfig.h"
#include "Widgets/SUserAssetTagBrowserSelectedAssetDetails.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/STaggedAssetBrowserSections.h"



#define LOCTEXT_NAMESPACE "TaggedAssetBrowser"

void STaggedAssetBrowser::Construct(const FArguments& InArgs, const UTaggedAssetBrowserConfiguration& InTaggedAssetBrowserConfiguration)
{
	Configuration = &InTaggedAssetBrowserConfiguration;
	TaggedBrowserProfileName = Configuration->ProfileName;

	ThisContext.Reset(NewObject<UTaggedAssetBrowserMenuContext>());
	ThisContext->TaggedAssetBrowser = SharedThis(this).ToWeakPtr();
	
	// We duplicate the root here as we manipulate data of the filters.
	FilterHierarchyRoot.Reset(Cast<UTaggedAssetBrowserFilterRoot>(StaticDuplicateObject(Configuration->FilterRoot, GetTransientPackage())));

	ApplyFilterExtensions();
	
	AvailableClasses = InArgs._AvailableClasses;
	DefaultDetailsTabConfiguration = InArgs._DefaultDetailsTabConfiguration;
	CustomTabInfos = InArgs._CustomTabInfos;
	RecentAndFavoritesList = InArgs._RecentAndFavoritesList;
	AssetSelectionMode = InArgs._AssetSelectionMode;
	OnAssetSelectedDelegate = InArgs._OnAssetSelected;
	OnAssetsActivatedDelegate = InArgs._OnAssetsActivated;
	OnGetExtraFrontendFiltersDelegate = InArgs._OnGetExtraFrontendFilters;
	OnExtendAddFilterMenuDelegate = InArgs._OnExtendAddFilterMenu;
	OnOverrideAssetPickerConfigDelegate = InArgs._OnOverrideAssetPickerConfig;
	OnActiveSectionChangedDelegate = InArgs._OnActiveSectionChanged;
	OnPrimaryFilterSelectionChangedDelegate = InArgs._OnPrimaryFilterSelectionChanged;
	InterfaceOverrideProfiles = InArgs._InterfaceOverrideProfiles;
	OnGenerateAssetTooltipWidgetDelegate = InArgs._OnGenerateAssetTooltipWidget;
	OnIsAssetValidForCustomToolTipDelegate = InArgs._OnIsAssetValidForCustomToolTip;
	PropertyHandle = InArgs._PropertyHandle;
	AdditionalReferencingAssets = InArgs._AdditionalReferencingAssets;

	CheckValidSetup();
	
	InitializeValues();

	ChildSlot
	[
		SNew(SBorder)
		.Padding(0.f)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().Expose(SectionsSlot)
				.AutoWidth()
				+ SHorizontalBox::Slot().Expose(AssetBrowserAreaSplitterSlot)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
				.Orientation(Orient_Horizontal)
				.Thickness(2.f)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Expose(AdditionalWidgetSlot)
		]
	];

	bSuppressSaveAndLoad = true;
	PopulateAssetBrowserSplitterSlot();
	PopulateAssetBrowserContentSlot();
	PopulateSectionsSlot();
	PopulatePrimaryFiltersSlot();

	if (NeedsCustomTabArea())
	{
		PopulateCustomTabsSlot();
	}

	if(InArgs._AdditionalBottomWidget.IsValid())
	{
		AdditionalWidgetSlot->AttachWidget(InArgs._AdditionalBottomWidget.ToSharedRef());
	}

	bSuppressSaveAndLoad = false;
	LoadSettings();
	
	InitAssetContextMenu();

	UTaggedAssetBrowserConfig::Get()->OnPropertyChanged().AddSP(this, &STaggedAssetBrowser::OnAssetBrowserConfigPropertyChanged);
}

STaggedAssetBrowser::STaggedAssetBrowser()
{
}

STaggedAssetBrowser::~STaggedAssetBrowser()
{
	SaveSettings();
	if (UTaggedAssetBrowserConfig::HasInstance())
	{
		UTaggedAssetBrowserConfig::Get()->OnPropertyChanged().RemoveAll(this);
	}
}

TArray<UClass*> STaggedAssetBrowser::GetDisplayedAssetTypes() const
{
	return AvailableClasses;
}

TAttribute<const FMainMRUFavoritesList*> STaggedAssetBrowser::GetFavoritesListAttribute() const
{
	return RecentAndFavoritesList;
}

TSharedPtr<SAssetPicker> STaggedAssetBrowser::GetAssetPicker() const
{
	return AssetBrowserContent.IsValid() ? AssetBrowserContent->GetAssetPicker() : nullptr;
}

void STaggedAssetBrowser::InitializeValues()
{
	PrimaryFiltersSlotValues.DefaultMinSize = 50.f;
	PrimaryFiltersSlotValues.DefaultSizeValue = 0.15f;

	// If not supplied, we set it to return false by default.
	// Since we always bind the custom tooltip widget delegate, if unbound, we will _always_ attempt to provide a custom tooltip, even if the Tagged Asset Browser has not been configured correctly
	// Explicitly returning false allows us to fall back to the content browser widgets
	if(OnIsAssetValidForCustomToolTipDelegate.IsBound() == false)
	{
		OnIsAssetValidForCustomToolTipDelegate = FOnIsAssetValidForCustomToolTip::CreateLambda([](FAssetData& AssetData)
		{
			return false;
		});
	}
}

void STaggedAssetBrowser::RefreshPrimaryFilterRootItems(bool bRefreshWidget)
{
	PrimaryFilterRootItems.Empty();
	
	TArray<TObjectPtr<UHierarchyElement>> Children(FilterHierarchyRoot->GetChildren());
	// If we have an active section, we only let children pass with a given section association
	if(SectionsSlotWidget.IsValid() && SectionsSlotWidget->GetActiveSection() != nullptr)
	{
		Children.RemoveAll([this](UHierarchyElement* Candidate)
		{
			FDataHierarchyElementMetaData_SectionAssociation SectionAssociation = Candidate->FindMetaDataOfTypeOrDefault<FDataHierarchyElementMetaData_SectionAssociation>();
			return SectionAssociation.Section != SectionsSlotWidget->GetActiveSection();
		});
	}

	PrimaryFilterRootItems.Append(Children);

	if(bRefreshWidget && PrimaryFilterSelector.IsValid())
	{
		PrimaryFilterSelector->SetTreeItemsSource(&PrimaryFilterRootItems);
		PrimaryFilterSelector->RequestTreeRefresh();

		if(PrimaryFilterRootItems.Num() == 0)
		{
			PrimaryFiltersSlot->AttachWidget(SNullWidget::NullWidget);
			PrimaryFiltersSlot->SetResizable(false);
		}
		else
		{
			PrimaryFiltersSlot->AttachWidget(PrimaryFiltersSlotWidget.ToSharedRef());
			PrimaryFiltersSlot->SetResizable(true);
		}
	}
}

void STaggedAssetBrowser::ApplyFilterExtensions()
{
	if(TaggedBrowserProfileName.IsSet() && TaggedBrowserProfileName->IsValid())
	{
		TMap<FName, const UHierarchySection*> MergeAllowedSectionMap;
		TMap<FName, const UHierarchySection*> MergeDisabledSectionMap;

		// Build a map of sections already present in the base config so we can merge into them
		TStrongObjectPtr<const UTaggedAssetBrowserConfiguration> Config = Configuration.Pin();
		bool bMergeExtensions = Config && Config->StandaloneData.bMergeExtensionsPerSection;
		if (bMergeExtensions)
		{
			for (const UHierarchySection* ExistingSection : FilterHierarchyRoot->GetSectionData())
			{
				const UTaggedAssetBrowserSection* TaggedAssetBrowserSection = Cast<UTaggedAssetBrowserSection>(ExistingSection);
				if (TaggedAssetBrowserSection && TaggedAssetBrowserSection->bAllowSectionMerge)
				{
					MergeAllowedSectionMap.Add(ExistingSection->GetSectionName(), ExistingSection);
				}
				else
				{
					MergeDisabledSectionMap.Add(ExistingSection->GetSectionName(), ExistingSection);
				}
			}
		}

		for(const UTaggedAssetBrowserConfiguration* ExtensionConfigurationAsset : GetExtensionAssets())
		{
			for(const UHierarchySection* Section : ExtensionConfigurationAsset->FilterRoot->GetSectionData())
			{
				const FName SectionName = Section->GetSectionName();
				if(!MergeAllowedSectionMap.Contains(SectionName) || !bMergeExtensions)
				{
					// No matching section in the base config — add a new one
					UHierarchySection* CopiedSection = Cast<UHierarchySection>(StaticDuplicateObject(Section, FilterHierarchyRoot.Get()));
					FilterHierarchyRoot->GetSectionDataMutable().Add(CopiedSection);

					const UTaggedAssetBrowserSection* TaggedAssetBrowserSection = Cast<UTaggedAssetBrowserSection>(CopiedSection);
					if (TaggedAssetBrowserSection && TaggedAssetBrowserSection->bAllowSectionMerge)
					{
						MergeAllowedSectionMap.Add(CopiedSection->GetSectionName(), CopiedSection);
					}
					else
					{
						MergeDisabledSectionMap.Add(CopiedSection->GetSectionName(), CopiedSection);
					}
				}
				else
				{
					// Do nothing. If the section already exists, we will add the elements to it below
				}
			}

			// We copy over the children and append them to the root
			for(const UHierarchyElement* Element : ExtensionConfigurationAsset->FilterRoot->GetChildren())
			{
				// Manually insert into appropriate location 
				UHierarchyElement* NewChild = Cast<UHierarchyElement>(StaticDuplicateObject(Element, FilterHierarchyRoot.Get()));
				if (NewChild->GetPersistentIdentity() != Element->GetPersistentIdentity())
				{
					check(false);
				}

				bool bFilterInserted = false;
				if (const UTaggedAssetBrowserFilterBase* Filter = Cast<UTaggedAssetBrowserFilterBase>(Element))
				{
					auto IsInsertTargetFilter = [FilterName = Filter->ExtensionData.InsertFilterName](const UHierarchyElement* ExistingElement)
					{
						if (const UTaggedAssetBrowserFilterBase* ExistingFilter = Cast<UTaggedAssetBrowserFilterBase>(ExistingElement))
						{
							return ExistingFilter->ToString() == FilterName.ToString();
						}

						return false;
					};

					if (bMergeExtensions 
						&& Filter->ExtensionData.MergeBehavior == ETaggedAssetBrowserFilterMergeBehavior::Insert
						&& !Filter->ExtensionData.InsertFilterName.IsNone())
					{
						int32 InsertIndex = FilterHierarchyRoot->GetChildrenMutable().IndexOfByPredicate(IsInsertTargetFilter);

						if (InsertIndex != INDEX_NONE)
						{
							if (Filter->ExtensionData.InsertBehavior == ETaggedAssetBrowserFilterInsertBehavior::After)
							{
								InsertIndex++;
							}
						}
						else
						{
							InsertIndex = FilterHierarchyRoot->GetChildrenMutable().Num();
						}

						FilterHierarchyRoot->GetChildrenMutable().Insert(NewChild, InsertIndex);
						bFilterInserted = true;
					}
				}

				if (!bFilterInserted)
				{
					FilterHierarchyRoot->GetChildrenMutable().Add(NewChild);
				}

				// We also need to fix up section associations
				FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = NewChild->FindMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
				if(SectionAssociation && SectionAssociation->Section.IsValid())
				{
					if(MergeAllowedSectionMap.Contains(SectionAssociation->Section->GetSectionName()))
					{
						SectionAssociation->Section = MergeAllowedSectionMap[SectionAssociation->Section->GetSectionName()];
					}
					else if (MergeDisabledSectionMap.Contains(SectionAssociation->Section->GetSectionName()))
					{
						SectionAssociation->Section = MergeDisabledSectionMap[SectionAssociation->Section->GetSectionName()];
					}
					else
					{
						UE_LOGFMT(LogUserAssetTags, Warning, "While fixing up the section linkage, no section with name {0} was found among the new copied sections. Filter won't work with sections correctly.", SectionAssociation->Section->GetSectionName());
					}
				}
			}
		}
	}
}

void STaggedAssetBrowser::InitAssetContextMenu()
{
	FName AssetContextMenuName = GetAssetContextMenuName();
	if(UToolMenus::Get()->IsMenuRegistered(AssetContextMenuName) == false)
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(AssetContextMenuName);
		FToolMenuSection& DefaultSection = Menu->AddSection("Default");

		FToolUIAction ToolUIAction;
		FToolMenuExecuteAction BrowseToAssetAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
		{
			if(UContentBrowserAssetContextMenuContext* ContentBrowserContext = Context.FindContext<UContentBrowserAssetContextMenuContext>())
			{
				if(GEditor)
				{
					GEditor->SyncBrowserToObjects(ContentBrowserContext->SelectedAssets);
				}
			}
		});

		FToolMenuCanExecuteAction CanBrowseToAssetAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext& Context)
		{
			UTaggedAssetBrowserMenuContext* TaggedAssetBrowserMenuContext = Context.FindContext<UTaggedAssetBrowserMenuContext>();
			if (TaggedAssetBrowserMenuContext == nullptr || !TaggedAssetBrowserMenuContext->TaggedAssetBrowser.IsValid())
			{
				return false;
			}
				
			TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(TaggedAssetBrowserMenuContext->TaggedAssetBrowser.Pin().ToSharedRef());
			if(Window && Window->IsModalWindow())
			{
				return false;
			}

			if(UContentBrowserAssetContextMenuContext* ContentBrowserContext = Context.FindContext<UContentBrowserAssetContextMenuContext>())
			{
				return ContentBrowserContext->SelectedAssets.Num() == 1;
			}

			return false;
		});
		
		FToolMenuIsActionButtonVisible IsBrowseToAssetVisible = FToolMenuIsActionButtonVisible::CreateLambda([CanBrowseToAssetAction](const FToolMenuContext& Context)
		{
			return CanBrowseToAssetAction.Execute(Context);
		});
		
		ToolUIAction.ExecuteAction = BrowseToAssetAction;
		ToolUIAction.CanExecuteAction = CanBrowseToAssetAction;
		ToolUIAction.IsActionVisibleDelegate = IsBrowseToAssetVisible;
		
		DefaultSection.AddEntry(FToolMenuEntry::InitMenuEntry("NavigateTo", LOCTEXT("NavigateTo", "Find in Content Browser"), FText::GetEmpty(), FSlateIcon(), ToolUIAction));
	}
}

void STaggedAssetBrowser::CheckValidSetup() const
{
	if(OnIsAssetValidForCustomToolTipDelegate.IsBound())
	{
		ensureMsgf(OnGenerateAssetTooltipWidgetDelegate.IsBound(), TEXT("If IsAssetValidForCustomToolTip is bound, you also have to bind OnGenerateAssetTooltipWidget."));
	}

	if(OnGenerateAssetTooltipWidgetDelegate.IsBound())
	{
		ensureMsgf(OnIsAssetValidForCustomToolTipDelegate.IsBound(), TEXT("If you want to use a custom tooltip, you also need to supply OnIsAssetValidForCustomToolTip. Falling back to Content Browser default tooltips."));
	}
	
	if(DefaultDetailsTabConfiguration.bUseDefaultDetailsTab)
	{
		for(UClass* Class : GetDisplayedAssetTypes())
		{
			ensureMsgf(FTaggedAssetBrowserDetailsDisplayDatabase::Data.Contains(Class), TEXT("If you use the default details tab, make sure to register class %s in the FUserAssetDetailsDatabase."), *(Class->GetFName().ToString()));
		}
	}
}

FARFilter STaggedAssetBrowser::CreateCurrentBackendFilter() const
{
	FARFilter Filter;
	Filter.bRecursivePaths = true;
	
	if(FilterHierarchyRoot.IsValid())
	{
		// If we have a valid section, use its given directories as filters
		if(SectionsSlotWidget.IsValid() && SectionsSlotWidget->GetActiveSection() != nullptr)
		{
			FARFilter SectionARFilter;

			for(const TObjectPtr<UTaggedAssetBrowserFilterBase>& SectionFilter : SectionsSlotWidget->GetActiveSection()->Filters)
			{
				if(SectionFilter != nullptr)
				{
					SectionFilter->ModifyARFilter(SectionARFilter);
				}
			}

			Filter.Append(SectionARFilter);
		}

		if(PrimaryFilterSelector.IsValid())
		{
			FARFilter PrimaryFilterARFilter;

			for(const TObjectPtr<UHierarchyElement>& PrimaryFilter : PrimaryFilterSelector->GetSelectedItems())
			{
				CastChecked<UTaggedAssetBrowserFilterBase>(PrimaryFilter)->ModifyARFilter(PrimaryFilterARFilter);
			}

			Filter.Append(PrimaryFilterARFilter);
		}
	}
	
	// if the filter list hasn't specified any classes of the available ones, we explicitly set the filter to the available classes.
	// this is because if no classes are specified, we could potentially receive assets outside the available classes
	if(Filter.ClassPaths.IsEmpty())
	{
		for(UClass* AvailableClass : AvailableClasses)
		{
			Filter.ClassPaths.Add(AvailableClass->GetClassPathName());
		}
	}

	return Filter;
}

bool STaggedAssetBrowser::ShouldFilterAsset(const FAssetData& AssetData) const
{
	if(OnAdditionalShouldFilterAssetDelegate.IsBound())
	{
		if(OnAdditionalShouldFilterAssetDelegate.Execute(AssetData))
		{
			return true;
		}
	}

	if(UTaggedAssetBrowserConfig::Get()->bShowHiddenAssets == false && UE::UserAssetTags::HasUserAssetTag(AssetData, "Hidden"))
	{
		return true;
	}
	
	if(UTaggedAssetBrowserConfig::Get()->bShowDeprecatedAssets == false && UE::UserAssetTags::HasUserAssetTag(AssetData, "Deprecated"))
	{
		return true;
	}

	if(FilterHierarchyRoot.IsValid())
	{
		if(SectionsSlotWidget.IsValid() && SectionsSlotWidget->GetActiveSection() != nullptr)
		{
			for(const TObjectPtr<UTaggedAssetBrowserFilterBase>& SectionFilter : SectionsSlotWidget->GetActiveSection()->Filters)
			{
				if(SectionFilter != nullptr)
				{
					if(SectionFilter->ShouldFilterAsset(AssetData))
					{
						return true;
					}
				}
			}
		}
		
		if(PrimaryFilterSelector.IsValid())
		{
			for(const TObjectPtr<UHierarchyElement>& Filter : PrimaryFilterSelector->GetSelectedItems())
			{
				if(Cast<UTaggedAssetBrowserFilterBase>(Filter)->ShouldFilterAsset(AssetData))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool STaggedAssetBrowser::NeedsCustomTabArea() const
{
	return DefaultDetailsTabConfiguration.bUseDefaultDetailsTab || !CustomTabInfos.IsEmpty();
}

void STaggedAssetBrowser::RefreshBackendFilter() const
{
	AssetBrowserContent->SetARFilter(CreateCurrentBackendFilter());
}

void STaggedAssetBrowser::PopulateSectionsSlot()
{
	if(FilterHierarchyRoot.IsValid() == false || FilterHierarchyRoot->GetSectionData().Num() == 0)
	{
		SectionsSlot->SetMinWidth(0.f);
	}
	else
	{
		SectionsSlot->SetAutoWidth();
		SectionsSlot->SetMinWidth(64.f);
	}
	
	if(FilterHierarchyRoot->GetSectionData().Num() >= 1)
	{
		SectionsSlotWidget = SNew(STaggedAssetBrowserSections, *FilterHierarchyRoot.Get())
			.OnSectionSelected(this, &STaggedAssetBrowser::OnSectionSelected)
			.InitiallyActiveSection(CastChecked<UTaggedAssetBrowserSection>(FilterHierarchyRoot->GetSectionData()[0]));
		SectionsSlot->AttachWidget(SectionsSlotWidget.ToSharedRef());
	}
}

void STaggedAssetBrowser::PopulatePrimaryFiltersSlot()
{
	if(FilterHierarchyRoot.IsValid() == false)
	{
		PrimaryFiltersSlot->SetMinSize(0.f);
		PrimaryFiltersSlot->SetResizable(false);
		PrimaryFiltersSlot->SetSizeValue(0.f);
		return;
	}
	
	InitializeFilters();

	RefreshPrimaryFilterRootItems(false);

	SAssignNew(PrimaryFilterSelector, STreeView<TObjectPtr<UHierarchyElement>>)
		.TreeItemsSource(&PrimaryFilterRootItems)
		.OnSelectionChanged(this, &STaggedAssetBrowser::OnPrimaryFilterSelected)
		.OnGenerateRow(this, &STaggedAssetBrowser::GenerateWidgetRowForPrimaryFilter)
		.OnGetChildren(this, &STaggedAssetBrowser::OnGetChildFiltersForFilter)
		.SelectionMode(ESelectionMode::Single);

	// If there is an All filter, select it by default
	TArray<UTaggedAssetBrowserFilter_All*> FoundAllFilters;
	FilterHierarchyRoot->GetChildrenOfType(FoundAllFilters);
	
	if(FoundAllFilters.Num() > 0)
	{
		PrimaryFilterSelector->SetSelection(FoundAllFilters[0]);
		PrimaryFilterSelector->SetItemExpansion(FoundAllFilters[0], true);
	}

	TSharedRef<SSearchBox> SearchBox = SNew(SSearchBox)
		.OnTextChanged(this, &STaggedAssetBrowser::OnFilterSearchTextChanged)
		.OnTextCommitted(this, &STaggedAssetBrowser::OnFilterSearchTextCommitted)
		.OnSearch(SSearchBox::FOnSearch::CreateSP(this, &STaggedAssetBrowser::OnSearchButtonClicked))
		.SearchResultData(this, &STaggedAssetBrowser::GetSearchResultData)
		.DelayChangeNotificationsWhileTyping(true);

	PrimaryFiltersSlotWidget = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.f)
		[
			SearchBox
		]
		+ SVerticalBox::Slot()
		.Padding(1.f, 2.f)
		[
			PrimaryFilterSelector.ToSharedRef()	
		];

	PrimaryFiltersSlotWidget->EnableToolTipForceField(true);
	
	PrimaryFiltersSlot->AttachWidget(PrimaryFiltersSlotWidget.ToSharedRef());
}

void STaggedAssetBrowser::PopulateAssetBrowserSplitterSlot()
{
	// @TODO: The splitter logic here is duplicated since SSplitter currently doesn't support 
	// after creation slot modification like horizontal / vertical box do.

	if (NeedsCustomTabArea())
	{
		AssetBrowserAreaSplitterSlot->AttachWidget(SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			.PhysicalSplitterHandleSize(2.f)
			+ SSplitter::Slot().Expose(PrimaryFiltersSlot)
			.Value(this, &STaggedAssetBrowser::OnGetPrimaryFiltersSlotSizeValue)
			.MinSize(this, &STaggedAssetBrowser::OnGetPrimaryFiltersSlotMinSize)
			.OnSlotResized(this, &STaggedAssetBrowser::OnPrimaryFiltersSlotResized)
			+ SSplitter::Slot().Expose(AssetBrowserContentSlot)
			.Value(0.6f)
			+ SSplitter::Slot().Expose(CustomTabsSlot)
			.Value(0.25f)
			.MinSize(100.f));
	}
	else
	{
		AssetBrowserAreaSplitterSlot->AttachWidget(SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			.PhysicalSplitterHandleSize(2.f)
			+ SSplitter::Slot().Expose(PrimaryFiltersSlot)
			.Value(this, &STaggedAssetBrowser::OnGetPrimaryFiltersSlotSizeValue)
			.MinSize(this, &STaggedAssetBrowser::OnGetPrimaryFiltersSlotMinSize)
			.OnSlotResized(this, &STaggedAssetBrowser::OnPrimaryFiltersSlotResized)
			+ SSplitter::Slot().Expose(AssetBrowserContentSlot)
			.Value(0.85f));
	}
}

void STaggedAssetBrowser::PopulateAssetBrowserContentSlot()
{
	FAssetPickerConfig Config;
	Config.Filter = CreateCurrentBackendFilter();
	Config.bCanShowClasses = false;
	Config.bCanShowFolders = false;
	Config.bAddFilterUI = true;
	Config.DefaultFilterMenuExpansion = InterfaceOverrideProfiles.DefaultFilterMenuExpansion.IsSet() ? InterfaceOverrideProfiles.DefaultFilterMenuExpansion.GetValue() : EAssetTypeCategories::Basic;
	Config.ExtraFrontendFilters = OnGetExtraFrontendFilters();
	Config.OnExtendAddFilterMenu = OnExtendAddFilterMenuDelegate;
	Config.bUseSectionsForCustomFilterCategories = true;
	Config.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &STaggedAssetBrowser::ShouldFilterAsset);
	Config.OnAssetSelected = FOnAssetSelected::CreateSP(this, &STaggedAssetBrowser::OnAssetSelected);
	Config.OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &STaggedAssetBrowser::OnAssetsActivated);
	Config.OnGetCustomAssetToolTip = FOnGetCustomAssetToolTip::CreateSP(this, &STaggedAssetBrowser::OnGetCustomAssetTooltip);
	Config.OnIsAssetValidForCustomToolTip = OnIsAssetValidForCustomToolTipDelegate;
	Config.bForceShowEngineContent = true;
	Config.bForceShowPluginContent = true;
	Config.SelectionMode = AssetSelectionMode;
	Config.bAllowDragging = false;
	Config.AssetViewOptionsProfile = InterfaceOverrideProfiles.AssetViewOptionsProfileName.IsSet() ? InterfaceOverrideProfiles.AssetViewOptionsProfileName.GetValue() : TaggedBrowserProfileName;
	Config.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &STaggedAssetBrowser::OnGetAssetContextMenu);
	Config.PropertyHandle = PropertyHandle;
	Config.AdditionalReferencingAssets = AdditionalReferencingAssets;

	if(TaggedBrowserProfileName.IsSet())
	{
		Config.SaveSettingsName = TaggedBrowserProfileName.GetValue().ToString();
	}

	Config.FilterBarSaveSettingsNameOverride = InterfaceOverrideProfiles.FilterBarSaveName.IsSet() ? InterfaceOverrideProfiles.FilterBarSaveName.GetValue().ToString() : TOptional<FString>();

	OnOverrideAssetPickerConfigDelegate.ExecuteIfBound(Config);

	SAssignNew(AssetBrowserContent, STaggedAssetBrowserContent)
	.InitialConfig(Config);
	
	AssetBrowserContentSlot->AttachWidget(AssetBrowserContent.ToSharedRef());
}

void STaggedAssetBrowser::PopulateCustomTabsSlot()
{
	TSharedRef<SWidgetSwitcher> TabSwitcher = SNew(SWidgetSwitcher);

	// If we want to use the default details tab, we construct a custom tab for it and configure it appropriately.
	if(DefaultDetailsTabConfiguration.bUseDefaultDetailsTab)
	{
		DefaultDetailsTabMembers = MakeShared<FDefaultDetailsTabMembers>();
		
		FTaggedAssetBrowserCustomTabInfo DefaultTabInfo;
		DefaultTabInfo.Title = LOCTEXT("DetailsTab", "Details");
		DefaultTabInfo.Icon = FAppStyle::GetBrush("Icons.Details");
		// The actual widgets constructed for the default details.
		DefaultTabInfo.OnGetTabContent = FOnGetContent::CreateLambda([this]() -> TSharedRef<SWidget>
		{
			SAssignNew(DefaultDetailsTabMembers->DefaultDetailsSwitcher, SWidgetSwitcher)
			+ SWidgetSwitcher::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.Padding(30.f, 60.f)
			[
				SNew(STextBlock)
				.Text(DefaultDetailsTabConfiguration.EmptySelectionMessage.Get(FText::GetEmpty()))
				.AutoWrapText(true)
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText.Subdued"))
			]
			+ SWidgetSwitcher::Slot()
			.Expose(DefaultDetailsTabMembers->AssetBrowserDefaultDetailsSlot);

			return DefaultDetailsTabMembers->DefaultDetailsSwitcher.ToSharedRef();
		});
		// When an asset gets selected, we update the contents of the default details slot
		DefaultTabInfo.OnAssetSelected = FOnAssetSelected::CreateLambda([this](const FAssetData& AssetData) -> void
		{
			if(AssetData.IsValid())
			{		
				TSharedPtr<SWidget> DetailsWidget = nullptr;

				// We construct the default asset details widget based on the DisplayDatabase. It also allows for the override of just the thumbnail.
				if(FTaggedAssetBrowserDetailsDisplayDatabase::Data.Contains(AssetData.GetClass()))
				{		
					DetailsWidget = SNew(SUserAssetTagBrowserSelectedAssetDetails, AssetData)
					.ShowThumbnailSlotWidget(TAttribute<EVisibility>::CreateSP(DefaultDetailsTabMembers.Get(), &FDefaultDetailsTabMembers::OnGetThumbnailVisibility))
					.OnGenerateThumbnailReplacementWidget(DefaultDetailsTabConfiguration.OnGenerateAssetThumbnailOverrideWidgetDelegate);
				}
				else
				{
					ensureMsgf(false, TEXT("This should never happen. Register display information for your class in FTaggedAssetBrowserDetailsDisplayDatabase."));
					DetailsWidget = SNullWidget::NullWidget;
				}
				
				DefaultDetailsTabMembers->AssetBrowserDefaultDetailsSlot->AttachWidget(DetailsWidget.ToSharedRef());
				DefaultDetailsTabMembers->DefaultDetailsSwitcher->SetActiveWidgetIndex(1);
			}
			else
			{
				// We remove the previously constructed details widget for good measure, and then switch to the empty message
				DefaultDetailsTabMembers->AssetBrowserDefaultDetailsSlot->AttachWidget(SNullWidget::NullWidget);
				DefaultDetailsTabMembers->DefaultDetailsSwitcher->SetActiveWidgetIndex(0);
			}
		});

		CustomTabInfos.Insert(DefaultTabInfo, 0);
	}

	for(const FTaggedAssetBrowserCustomTabInfo& TabInfo : CustomTabInfos)
	{
		TabSwitcher->AddSlot()
		[
			TabInfo.OnGetTabContent.Execute()
		];
	}

	TSharedRef<SSegmentedControl<int32>> TabSwitchControls = SNew(SSegmentedControl<int32>)
		.Visibility(CustomTabInfos.Num() > 1 ? EVisibility::Visible : EVisibility::Collapsed)
		.Value(0)
		.OnValueChanged_Lambda([Switcher = TabSwitcher](int32 NewValue)
		{
			Switcher->SetActiveWidgetIndex(NewValue);
		});

	for(auto It = CustomTabInfos.CreateConstIterator(); It; ++It)
	{
		TabSwitchControls->AddSlot(It.GetIndex())
		.Text(It->Title)
		.Icon(It->Icon);
	}
	
	TSharedRef<SVerticalBox> Widget = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			TabSwitchControls
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.HAlign(HAlign_Fill)
		[
			TabSwitcher
		];
	
	if (!CustomTabInfos.IsEmpty())
	{
		CustomTabsSlot->AttachWidget(Widget);
	}
}

void STaggedAssetBrowser::OnPrimaryFiltersSlotResized(float InSize)
{
	PrimaryFiltersSlotValues.CurrentSizeValue = InSize;
}

float STaggedAssetBrowser::OnGetPrimaryFiltersSlotMinSize() const
{
	if(PrimaryFilterRootItems.Num() == 0)
	{
		return 0.f;
	}

	return PrimaryFiltersSlotValues.DefaultMinSize;
}

float STaggedAssetBrowser::OnGetPrimaryFiltersSlotSizeValue() const
{
	if(PrimaryFilterRootItems.Num() == 0)
	{
		return 0.f;
	}

	return PrimaryFiltersSlotValues.CurrentSizeValue;
}

void STaggedAssetBrowser::InitializeFilters()
{
	using namespace UE::UserAssetTags;

	FTaggedAssetBrowserContext FilterContext;
	FilterContext.TaggedAssetBrowser = SharedThis(this);
	FilterContext.OnGetActiveSectionDelegate = FTaggedAssetBrowserContext::FOnGetActiveSection::CreateSP(this, &STaggedAssetBrowser::GetActiveSection);
	FilterContext.OnGetSelectedFilters = FTaggedAssetBrowserContext::FOnGetSelectedFilters::CreateSP(this, &STaggedAssetBrowser::GetSelectedPrimaryFilters);
	
	TArray<UTaggedAssetBrowserFilterBase*> AllFilters;
	FilterHierarchyRoot->GetChildrenOfType(AllFilters, true);

	for(UTaggedAssetBrowserFilterBase* Filter : AllFilters)
	{
		Filter->Initialize(FilterContext);
	}

	for(UHierarchySection* Section : FilterHierarchyRoot->GetSectionDataMutable())
	{
		UTaggedAssetBrowserSection* TaggedAssetBrowserSection = CastChecked<UTaggedAssetBrowserSection>(Section);
		for(UTaggedAssetBrowserFilterBase* Filter : TaggedAssetBrowserSection->Filters)
		{
			// Could be nullptr if the user added an entry but didn't set a class
			if(Filter)
			{
				Filter->Initialize(FilterContext);
			}
		}
	}
}

void STaggedAssetBrowser::OnFilterChanged() const
{
	RefreshBackendFilter();
}

void STaggedAssetBrowser::ExpandPrimaryFilterSearchResults()
{
	PrimaryFilterSelector->ClearExpandedItems();

	for(const FFilterPath& SearchResult : SourceSearchResults)
	{
		for(const TWeakObjectPtr<UHierarchyElement>& EntryInPath : SearchResult.Path)
		{
			PrimaryFilterSelector->SetItemExpansion(EntryInPath.Get(), true);
		}
	}
}

void STaggedAssetBrowser::SelectNextPrimaryFilterSearchResult()
{
	if(SourceSearchResults.IsEmpty())
	{
		return;
	}
	
	if(!FocusedSearchResult.IsSet())
	{
		FocusedSearchResult = SourceSearchResults[0];
	}
	else
	{
		int32 CurrentSearchResultIndex = SourceSearchResults.Find(FocusedSearchResult.GetValue());
		if(SourceSearchResults.IsValidIndex(CurrentSearchResultIndex+1))
		{
			FocusedSearchResult = SourceSearchResults[CurrentSearchResultIndex+1];
		}
		else
		{
			FocusedSearchResult = SourceSearchResults[0];
		}
	}

	PrimaryFilterSelector->ClearSelection();
	PrimaryFilterSelector->RequestScrollIntoView(FocusedSearchResult.GetValue().GetEntry().Get());
	PrimaryFilterSelector->SetItemSelection(FocusedSearchResult.GetValue().GetEntry().Get(), true);
}

void STaggedAssetBrowser::SelectPreviousPrimaryFilterSearchResult()
{
	if(SourceSearchResults.IsEmpty())
	{
		return;
	}
	
	if(!FocusedSearchResult.IsSet())
	{
		FocusedSearchResult = SourceSearchResults[0];
	}
	else
	{
		int32 CurrentSearchResultIndex = SourceSearchResults.Find(FocusedSearchResult.GetValue());
		if(SourceSearchResults.IsValidIndex(CurrentSearchResultIndex-1))
		{
			FocusedSearchResult = SourceSearchResults[CurrentSearchResultIndex-1];
		}
		else
		{
			FocusedSearchResult = SourceSearchResults[SourceSearchResults.Num()-1];
		}
	}

	PrimaryFilterSelector->ClearSelection();
	PrimaryFilterSelector->RequestScrollIntoView(FocusedSearchResult.GetValue().GetEntry().Get());
	PrimaryFilterSelector->SetItemSelection(FocusedSearchResult.GetValue().GetEntry().Get(), true);
}

void STaggedAssetBrowser::OnSectionSelected(const UTaggedAssetBrowserSection* InTaggedAssetBrowserSection)
{
	// Clear the selection since the new section likely doesn't have the currently selected filters
	if(PrimaryFilterSelector.IsValid())
	{
		PrimaryFilterSelector->ClearSelection();
	}
	
	RefreshPrimaryFilterRootItems(true);
	RefreshBackendFilter();
	OnActiveSectionChangedDelegate.ExecuteIfBound(InTaggedAssetBrowserSection);
}

const UTaggedAssetBrowserSection* STaggedAssetBrowser::GetActiveSection() const
{
	if(SectionsSlotWidget.IsValid())
	{
		return SectionsSlotWidget->GetActiveSection();
	}

	return nullptr;
}

TArray<const UTaggedAssetBrowserFilterBase*> STaggedAssetBrowser::GetSelectedPrimaryFilters() const
{
	TArray<const UTaggedAssetBrowserFilterBase*> Result;
	
	if(PrimaryFilterSelector.IsValid())
	{
		TArray<UHierarchyElement*> SelectedItems = PrimaryFilterSelector->GetSelectedItems();
		Algo::TransformIf(SelectedItems, Result, [](const UHierarchyElement* Element) -> bool
		{
			return Element->IsA<UTaggedAssetBrowserFilterBase>();
		},
	[](const UHierarchyElement* Element) -> const UTaggedAssetBrowserFilterBase*
		{
			return CastChecked<UTaggedAssetBrowserFilterBase>(Element);
		});		
	}

	return Result;
}

TArray<const UTaggedAssetBrowserFilterBase*> STaggedAssetBrowser::GetAllPrimaryFilters() const
{
	TArray<UTaggedAssetBrowserFilterBase*> AllFilters;
	FilterHierarchyRoot->GetChildrenOfType(AllFilters, true);

	// Second array for const
	TArray<const UTaggedAssetBrowserFilterBase*> Result;
	Result.Reserve(AllFilters.Num());
	for (UTaggedAssetBrowserFilterBase* Filter : AllFilters)
	{
		Result.Add(Filter);
	}
	return Result;
}

void STaggedAssetBrowser::GenerateSearchItems(TWeakObjectPtr<UHierarchyElement> Root, TArray<TWeakObjectPtr<UHierarchyElement>> ParentChain, TArray<FFilterPath>& OutSearchItems) const
{
	TConstArrayView<TObjectPtr<UHierarchyElement>> FilteredChildren = Root->GetChildren();
	
	ParentChain.Add(Root);
	OutSearchItems.Add(FFilterPath{ParentChain});
	for(TWeakObjectPtr<UHierarchyElement> Child : FilteredChildren)
	{
		GenerateSearchItems(Child, ParentChain, OutSearchItems);
	}
}

void STaggedAssetBrowser::OnFilterSearchTextChanged(const FText& Text)
{
	SourceSearchResults.Empty();
	FocusedSearchResult.Reset();
	PrimaryFilterSelector->ClearSelection();

	if(!Text.IsEmpty())
	{
		FText NoWhitespaceText = FText::FromString(Text.ToString().Replace(TEXT(" "), TEXT("")));

		TArray<UTaggedAssetBrowserFilterBase*> Filters;
		FilterHierarchyRoot->GetChildrenOfType(Filters);
		for(const TWeakObjectPtr<UTaggedAssetBrowserFilterBase> Filter : Filters)
		{
			TArray<FFilterPath> SearchItems;
			GenerateSearchItems(Filter, {}, SearchItems);

			for(const FFilterPath& SearchItem : SearchItems)
			{
				if(Cast<UTaggedAssetBrowserFilterBase>(SearchItem.GetEntry())->DoesFilterMatchTextQuery(NoWhitespaceText))
				{
					SourceSearchResults.Add(SearchItem);
				}
			}
		}

		ExpandPrimaryFilterSearchResults();
		SelectNextPrimaryFilterSearchResult();
	}
	else
	{
		PrimaryFilterSelector->ClearExpandedItems();
	}
}

void STaggedAssetBrowser::OnSearchButtonClicked(SSearchBox::SearchDirection SearchDirection)
{
	if(SearchDirection == SSearchBox::Next)
	{
		SelectNextPrimaryFilterSearchResult();
	}
	else
	{
		SelectPreviousPrimaryFilterSearchResult();
	}
}

void STaggedAssetBrowser::OnFilterSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	bool bIsShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	if(CommitType == ETextCommit::OnEnter)
	{
		if(bIsShiftDown == false)
		{
			SelectNextPrimaryFilterSearchResult();
		}
		else
		{
			SelectPreviousPrimaryFilterSearchResult();
		}
	}
}

TOptional<SSearchBox::FSearchResultData> STaggedAssetBrowser::GetSearchResultData() const
{
	if(SourceSearchResults.Num() > 0)
	{
		SSearchBox::FSearchResultData SearchResultData;
		SearchResultData.NumSearchResults = SourceSearchResults.Num();

		if(FocusedSearchResult.IsSet())
		{
			// we add one just to make it look nicer as this is merely for cosmetic purposes
			SearchResultData.CurrentSearchResultIndex = SourceSearchResults.Find(FocusedSearchResult.GetValue()) + 1;
		}
		else
		{
			SearchResultData.CurrentSearchResultIndex = INDEX_NONE;
		}

		return SearchResultData;
	}

	return TOptional<SSearchBox::FSearchResultData>();
}

void STaggedAssetBrowser::OnGetChildFiltersForFilter(TObjectPtr<UHierarchyElement> Filter, TArray<TObjectPtr<UHierarchyElement>>& OutChildren) const
{
	OutChildren.Append(Filter->GetChildren());
}

bool STaggedAssetBrowser::OnCompareFiltersForEquality(const UTaggedAssetBrowserFilterBase& FilterA, const UTaggedAssetBrowserFilterBase& FilterB) const
{
	return &FilterA == &FilterB;
}

TSharedRef<ITableRow> STaggedAssetBrowser::GenerateWidgetRowForPrimaryFilter(TObjectPtr<UHierarchyElement> PrimaryFilter, const TSharedRef<STableViewBase>& OwningTable) const
{
	// This function should generally mirror FTaggedAssetBrowserConfigurationEditor::GenerateFilterRowContent, but not rely on hierarchy view models
	if(UTaggedAssetBrowserFilterBase* Filter = Cast<UTaggedAssetBrowserFilterBase>(PrimaryFilter))
	{
		TSharedPtr<SHorizontalBox> ExtensionBox;

		TSharedRef<SWidget> Widget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SImage)
			.Image_UObject(Filter, &UTaggedAssetBrowserFilterBase::GetIconBrush)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(STextBlock)
			.Text(Filter->ToText())
		]
		// The extension box takes up the rest of the space, so that users can determine left/right alignment
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SAssignNew(ExtensionBox, SHorizontalBox)
		];
		
		Widget->SetToolTipText(Filter->GetTooltip());	

		Filter->CreateAdditionalWidgets(ExtensionBox);
		
		return SNew(STableRow<TObjectPtr<UTaggedAssetBrowserFilterBase>>, OwningTable)
		[
			Widget
		];
	}

	return SNew(STableRow<TObjectPtr<UTaggedAssetBrowserFilterBase>>, OwningTable)
		.Visibility(EVisibility::Collapsed)
		[
			SNullWidget::NullWidget
		];	
}

TArray<FAssetData> STaggedAssetBrowser::GetSelectedAssets() const
{
	return AssetBrowserContent->GetCurrentSelection();
}

void STaggedAssetBrowser::OnAssetSelected(const FAssetData& AssetData)
{
	for(FTaggedAssetBrowserCustomTabInfo& TabInfo : CustomTabInfos)
	{
		TabInfo.OnAssetSelected.ExecuteIfBound(AssetData);
	}

	OnAssetSelectedDelegate.ExecuteIfBound(AssetData);
}

void STaggedAssetBrowser::OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type ActivationType) const
{
	OnAssetsActivatedDelegate.ExecuteIfBound(AssetData, ActivationType);
}

TSharedPtr<SWidget> STaggedAssetBrowser::OnGetAssetContextMenu(const TArray<FAssetData>& AssetData) const
{
	if(AssetData.Num() == 1)
	{
		UContentBrowserAssetContextMenuContext* ContentBrowserContext = NewObject<UContentBrowserAssetContextMenuContext>();
		ContentBrowserContext->SelectedAssets = AssetData;
		
		FToolMenuContext Context;
		Context.AddObject(ContentBrowserContext);
		Context.AddObject(ThisContext.Get());
		return UToolMenus::Get()->GenerateWidget(GetAssetContextMenuName(), Context);
	}

	return nullptr;
}

TSharedRef<SToolTip> STaggedAssetBrowser::OnGetCustomAssetTooltip(FAssetData& AssetData)
{
	TSharedPtr<SWidget> TooltipContentWidget = nullptr;

	if(OnGenerateAssetTooltipWidgetDelegate.IsBound())
	{
		TooltipContentWidget = OnGenerateAssetTooltipWidgetDelegate.Execute(AssetData);
	}
	else if(FTaggedAssetBrowserDetailsDisplayDatabase::Data.Contains(AssetData.GetClass()))
	{
		TooltipContentWidget = SNew(SUserAssetTagBrowserSelectedAssetDetails, AssetData).ShowThumbnailSlotWidget(EVisibility::Collapsed);
	}
	else
	{
		ensureMsgf(false, TEXT("This should never happen. Either supply an OnGenerateAssetTooltipWidget binding, bind OnIsAssetValidForCustomTooltip and return false, or register information for your class in FTaggedAssetBrowserDetailsDisplayDatabase."));
		TooltipContentWidget = SNullWidget::NullWidget;
	}
	
	return SNew(SToolTip)
	[
		TooltipContentWidget.ToSharedRef()
	];
}

void STaggedAssetBrowser::OnPrimaryFilterSelected(TObjectPtr<UHierarchyElement> Filter, ESelectInfo::Type Arg)
{	
	OnFilterChanged();

	if(Filter)
	{
		LastSelectedPrimaryFilterIdentifierFallback = Cast<UTaggedAssetBrowserFilterBase>(Filter)->GetIdentifier();
	}

	SaveSettings();
	OnPrimaryFilterSelectionChangedDelegate.ExecuteIfBound();
}

TArray<TSharedRef<FFrontendFilter>> STaggedAssetBrowser::OnGetExtraFrontendFilters() const
{
	TArray<TSharedRef<FFrontendFilter>> Result;
	
	if(OnGetExtraFrontendFiltersDelegate.IsBound())
	{
		Result.Append(OnGetExtraFrontendFiltersDelegate.Execute());
	}
	
	return Result;
}

void STaggedAssetBrowser::OnAssetBrowserConfigPropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	OnFilterChanged();
}

void STaggedAssetBrowser::SaveSettings() const
{
	if(TaggedBrowserProfileName.IsSet() && bSuppressSaveAndLoad == false && !IsEngineExitRequested())
	{
		FPerTaggedAssetBrowserSavedState& Config = UTaggedAssetBrowserConfig::Get()->PerTaggedAssetBrowserSettings.Add(TaggedBrowserProfileName.GetValue());

		if(FilterHierarchyRoot.IsValid())
		{
			for(const TObjectPtr<UHierarchyElement>& SelectedPrimaryFilter : PrimaryFilterSelector->GetSelectedItems())
			{
				Config.PrimaryFilterSelection.Add(Cast<UTaggedAssetBrowserFilterBase>(SelectedPrimaryFilter)->GetIdentifier());
			}

			if(Config.PrimaryFilterSelection.Num() == 0 && LastSelectedPrimaryFilterIdentifierFallback != NAME_None)
			{
				Config.PrimaryFilterSelection.Add(LastSelectedPrimaryFilterIdentifierFallback);
			}
		}

		UTaggedAssetBrowserConfig::Get()->SaveEditorConfig();
	}
}

void STaggedAssetBrowser::LoadSettings()
{
	if(TaggedBrowserProfileName.IsSet() && bSuppressSaveAndLoad == false)
	{
		UTaggedAssetBrowserConfig::Get()->LoadEditorConfig();

		if(UTaggedAssetBrowserConfig::Get()->PerTaggedAssetBrowserSettings.Contains(FName(TaggedBrowserProfileName.GetValue())))
		{
			FPerTaggedAssetBrowserSavedState Config = UTaggedAssetBrowserConfig::Get()->PerTaggedAssetBrowserSettings[FName(TaggedBrowserProfileName.GetValue())];

			if(FilterHierarchyRoot.IsValid())
			{
				TArray<UTaggedAssetBrowserFilterBase*> AllFilters;
				FilterHierarchyRoot->GetChildrenOfType(AllFilters, true);

				UTaggedAssetBrowserFilterBase** FoundFilter = nullptr;
				if(Config.PrimaryFilterSelection.Num() > 0)
				{
					FoundFilter = AllFilters.FindByPredicate([&Config](const UTaggedAssetBrowserFilterBase* PrimaryFilterCandidate)
					{
						return Config.PrimaryFilterSelection[0] == PrimaryFilterCandidate->GetIdentifier();
					});
				}

				if(Config.PrimaryFilterSelection.Num() == 1)
				{
					LastSelectedPrimaryFilterIdentifierFallback = Config.PrimaryFilterSelection[0];
				}

				if(FoundFilter)
				{
					TArray<const TObjectPtr<UHierarchyElement>> Parents = (*FoundFilter)->GetParentElements();

					for(int32 ReverseIndex = Parents.Num() - 1; ReverseIndex >= 0; --ReverseIndex)
					{
						PrimaryFilterSelector->SetItemExpansion(Parents[ReverseIndex], true);
					}

					// Settings the primary filter selection will update the secondary filters
					PrimaryFilterSelector->SetSelection(*FoundFilter);
					PrimaryFilterSelector->RequestScrollIntoView(*FoundFilter);
				}
			}
		}
	}
}

TArray<UTaggedAssetBrowserConfiguration*> STaggedAssetBrowser::GetExtensionAssets() const
{
	TArray<UTaggedAssetBrowserConfiguration*> Result;

	if(TaggedBrowserProfileName.IsSet() && TaggedBrowserProfileName->IsValid())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		// We retrieve all extension configuration assets with the matching markup.
		FARFilter Filter;
		Filter.ClassPaths.Add(UTaggedAssetBrowserConfiguration::StaticClass()->GetClassPathName());
		Filter.TagsAndValues.Add(FName("bIsExtension"), FString("True"));
		Filter.TagsAndValues.Add(FName("ProfileName"), TaggedBrowserProfileName.GetValue().ToString());
		Filter.bIncludeOnlyOnDiskAssets = true;
			
		TArray<FAssetData> ExtensionAssets;
		AssetRegistryModule.Get().GetAssets(Filter, ExtensionAssets);

		for(const FAssetData& ExtensionConfigurationAssetData : ExtensionAssets)
		{
			// If the asset isn't visible in the user's context, we skip
			if(IAssetTools::Get().IsAssetVisible(ExtensionConfigurationAssetData, true) == false)
			{
				continue;
			}
			
			UTaggedAssetBrowserConfiguration* ExtensionConfigurationAsset = Cast<UTaggedAssetBrowserConfiguration>(ExtensionConfigurationAssetData.GetAsset());

			// The asset should have correct values but we make sure since we need to load it anyways
			if(ExtensionConfigurationAsset->bIsExtension == false || ExtensionConfigurationAsset->ProfileName != TaggedBrowserProfileName)
			{
				continue;
			}			

			Result.Add(ExtensionConfigurationAsset);
		}
	}
	
	return Result;
}

FName STaggedAssetBrowser::GetAssetContextMenuName() const
{
	static FName MenuName("TaggedAssetBrowser.AssetContextMenu.");
	FName InstanceMenuName = FName(MenuName.ToString() + (InterfaceOverrideProfiles.AssetViewContextMenuName.IsSet() ? InterfaceOverrideProfiles.AssetViewContextMenuName.GetValue() : TaggedBrowserProfileName.GetValue()).ToString());

	return InstanceMenuName;
}

EVisibility STaggedAssetBrowser::FDefaultDetailsTabMembers::OnGetThumbnailVisibility() const
{
	return EVisibility::Visible;
}

void STaggedAssetBrowserWindow::Construct(const FArguments& InArgs, const UTaggedAssetBrowserConfiguration& Configuration)
{	
	SWindow::Construct(InArgs._WindowArgs);

	SAssignNew(AssetBrowser, STaggedAssetBrowser, Configuration)
		.AvailableClasses(InArgs._AssetBrowserArgs._AvailableClasses)
		.DefaultDetailsTabConfiguration(InArgs._AssetBrowserArgs._DefaultDetailsTabConfiguration)
		.CustomTabInfos(InArgs._AssetBrowserArgs._CustomTabInfos)
		.InterfaceOverrideProfiles(InArgs._AssetBrowserArgs._InterfaceOverrideProfiles)
		.RecentAndFavoritesList(InArgs._AssetBrowserArgs._RecentAndFavoritesList)
		.OnGetExtraFrontendFilters(InArgs._AssetBrowserArgs._OnGetExtraFrontendFilters)
		.OnExtendAddFilterMenu(InArgs._AssetBrowserArgs._OnExtendAddFilterMenu)
		.OnAdditionalShouldFilterAsset(InArgs._AssetBrowserArgs._OnAdditionalShouldFilterAsset)
		.AssetSelectionMode(InArgs._AssetBrowserArgs._AssetSelectionMode)
		.OnAssetSelected(InArgs._AssetBrowserArgs._OnAssetSelected)
		.OnAssetsActivated(InArgs._AssetBrowserArgs._OnAssetsActivated)
		.OnGenerateAssetTooltipWidget(InArgs._AssetBrowserArgs._OnGenerateAssetTooltipWidget)
		.OnIsAssetValidForCustomToolTip(InArgs._AssetBrowserArgs._OnIsAssetValidForCustomToolTip)
		.AdditionalBottomWidget(InArgs._AssetBrowserArgs._AdditionalBottomWidget)
		.PropertyHandle(InArgs._AssetBrowserArgs._PropertyHandle)
		.AdditionalReferencingAssets(InArgs._AssetBrowserArgs._AdditionalReferencingAssets);
	
	SetContent(AssetBrowser.ToSharedRef());
}

bool STaggedAssetBrowserWindow::HasSelectedAssets() const
{
	return GetSelectedAssets().Num() > 0;
}

TArray<FAssetData> STaggedAssetBrowserWindow::GetSelectedAssets() const
{
	return AssetBrowser->GetSelectedAssets();
}

FReply STaggedAssetBrowserWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = SWindow::OnKeyDown(MyGeometry, InKeyEvent);

	if(InKeyEvent.GetKey() == EKeys::Escape)
	{
		RequestDestroyWindow();
	}

	return Reply;
}

#undef LOCTEXT_NAMESPACE


