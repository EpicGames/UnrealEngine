// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STaggedAssetBrowserAssetFactoryWindow.h"

#include "ContentBrowserModule.h"
#include "EditorDirectories.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "TaggedAssetBrowser"

void STaggedAssetBrowserAssetFactoryWindow::Construct(const FArguments& InArgs, const UTaggedAssetBrowserConfiguration& Configuration, UClass& InCreatedClass)
{
	CreatedClass = &InCreatedClass;

	if(!CheckValidArguments(InArgs))
	{
		SWindow::FArguments WindowArguments;
		SWindow::Construct(WindowArguments);
		return;
	}

	if(InArgs._bUseAssetDefinition)
	{
		AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(CreatedClass.Get());
	}

	FSlateApplication::Get().OnFocusChanging().AddSP(this, &STaggedAssetBrowserAssetFactoryWindow::CloseIfOtherWindowFocused);
	
	AsyncFactoryArguments = InArgs._AsyncFactoryArguments;
	AdditionalFactorySettingsClass = InArgs._AdditionalFactorySettingsClass;
	bAllowEmptyAssetCreation = InArgs._bAllowEmptyAssetCreation;

	FArguments Args = InArgs;

	TSharedRef<SWidget> CreateAssetControls = SNew(SBox)
		.Padding(16.f, 16.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.f, 1.f)
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.OnClicked(this, &STaggedAssetBrowserAssetFactoryWindow::Proceed)
				.Visibility(bAllowEmptyAssetCreation ? EVisibility::Visible : EVisibility::Hidden)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					[
						SNew(SImage)
						.Image(FSlateIconFinder::FindIconForClass(CreatedClass.Get()).GetIcon())
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					[
						SNew(STextBlock).Text(FText::FormatOrdered(LOCTEXT("CreateEmptyAssetButtonLabel", "Create Empty {0}"), GetAssetTypeName()))
					]
				]
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.f, 1.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(50.f, 4.f, 50.f, 1.f))
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PrimaryButtonText"))
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("CreatePrimaryButtonLabel", "Create"))
				.OnClicked(this, &STaggedAssetBrowserAssetFactoryWindow::Proceed)
				.IsEnabled(this, &STaggedAssetBrowserAssetFactoryWindow::HasSelectedAssets)
				.ToolTipText(this, &STaggedAssetBrowserAssetFactoryWindow::GetCreateButtonTooltip)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.f, 1.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(50.f, 4.f, 50.f, 1.f))
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PrimaryButtonText"))
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
				.OnClicked(this, &STaggedAssetBrowserAssetFactoryWindow::Cancel)
			]
		];

	if(AdditionalFactorySettingsClass.IsValid())
	{
		FTaggedAssetBrowserCustomTabInfo FactoryTabInfo;
		FactoryTabInfo.Title = LOCTEXT("FactoryTabTitle", "Settings");
		FactoryTabInfo.Icon = FAppStyle::GetBrush("Icons.Settings");
		FactoryTabInfo.OnGetTabContent = FOnGetContent::CreateSP(this, &STaggedAssetBrowserAssetFactoryWindow::CreateFactorySettingsTab);

		Args._AssetBrowserWindowArgs._AssetBrowserArgs._CustomTabInfos = { FactoryTabInfo };
	}
	
	Args._AssetBrowserWindowArgs._AssetBrowserArgs._AdditionalBottomWidget = CreateAssetControls;
	Args._AssetBrowserWindowArgs._AssetBrowserArgs._OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &STaggedAssetBrowserAssetFactoryWindow::OnAssetsActivated);

	// If we haven't specified any referencing assets, we assume the context is the last "save asset" editor path, which should be set before this is called
	if(Args._AssetBrowserWindowArgs._AssetBrowserArgs._AdditionalReferencingAssets.Num() == 0)
	{
		FString PackagePath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
		FString AssetName = "TmpAsset";
		FName PackageName = FName(PackagePath + "/" + AssetName);
		FAssetData PseudoReferencingAsset(PackageName, FName(PackagePath), FName(AssetName), UObject::StaticClass()->GetClassPathName());
		Args._AssetBrowserWindowArgs._AssetBrowserArgs._AdditionalReferencingAssets = { PseudoReferencingAsset };  
	}
	
	STaggedAssetBrowserWindow::Construct(Args._AssetBrowserWindowArgs, Configuration);
}

STaggedAssetBrowserAssetFactoryWindow::~STaggedAssetBrowserAssetFactoryWindow()
{
	FSlateApplication::Get().OnFocusChanging().RemoveAll(this);
}

bool STaggedAssetBrowserAssetFactoryWindow::CheckValidArguments(const FArguments& InArgs) const
{
	if(!ensureMsgf(!InArgs._AssetBrowserWindowArgs._AssetBrowserArgs._OnAssetsActivated.IsBound(), TEXT("OnAssetsActivated should not be bound.")))
	{
		return false;
	}

	if(InArgs._bUseAssetDefinition)
	{		
		if(!ensureMsgf(UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(CreatedClass.Get()) != nullptr, TEXT("AssetDefinition should be valid if bUseAssetDefinition is true.")))
		{
			return false;
		}
	}

	return true;
}

void STaggedAssetBrowserAssetFactoryWindow::OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type ActivationMethod)
{
	// Asset activation will close this window, and resume the factory path to call UFactory::FactoryCreateNew.
	
	// If modal, the factory can check this bool for proceeding/canceling right within the ConfigureProperties function after the window closes.
	bProceedWithAction = true;

	// If non-modal, this will allow the factory to configure its properties based on asset activation.
	if(AsyncFactoryArguments.IsSet())
	{
		AsyncFactoryArguments->OnAssetsActivated.ExecuteIfBound(AssetData, ActivationMethod);
		AsyncFactoryArguments->OnFactoryConfigurePropertiesComplete.ExecuteIfBound(AsyncFactoryArguments->Factory.Get());
	}
	
	RequestDestroyWindow();
}

FReply STaggedAssetBrowserAssetFactoryWindow::Proceed()
{
	OnAssetsActivated(GetSelectedAssets(), EAssetTypeActivationMethod::Opened);
	return FReply::Handled();
}

FReply STaggedAssetBrowserAssetFactoryWindow::Cancel()
{
	bProceedWithAction = false;

	if(AsyncFactoryArguments.IsSet())
	{
		AsyncFactoryArguments->OnFactoryConfigurePropertiesCancelled.ExecuteIfBound(AsyncFactoryArguments->Factory.Get());
	}
	
	RequestDestroyWindow();
	return FReply::Handled();
}

FText STaggedAssetBrowserAssetFactoryWindow::GetAssetTypeName() const
{
	return AssetDefinition.IsValid() ? AssetDefinition->GetAssetDisplayName() : CreatedClass->GetDisplayNameText();
}

FText STaggedAssetBrowserAssetFactoryWindow::GetCreateButtonTooltip() const
{
	return HasSelectedAssets()
	? FText::FormatOrdered(LOCTEXT("CreateAssetButtonTooltip_Enabled", "Create a new {0} with selected asset {1}"), GetAssetTypeName(), FText::FromName(GetSelectedAssets()[0].AssetName))
	: FText::FormatOrdered(LOCTEXT("CreateAssetButtonTooltip_Disabled", "Please select an asset as a base for your new effect.{0}"), bAllowEmptyAssetCreation ? LOCTEXT("CreateAssetButtonTooltip_Disabled_AllowsEmptyCreation", " Alternatively, create an empty asset.") : FText::GetEmpty());
}

TSharedRef<SWidget> STaggedAssetBrowserAssetFactoryWindow::CreateFactorySettingsTab()
{
	if(AdditionalFactorySettingsClass.IsValid())
	{
		FactorySettingsObject = NewObject<UObject>(GetTransientPackage(), AdditionalFactorySettingsClass.Get(), NAME_None, RF_Transient);
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs Args;
		Args.bShowObjectLabel = false;
		Args.NameAreaSettings = FDetailsViewArgs::HideNameArea;

		FactorySettingsWidget = PropertyEditorModule.CreateDetailView(Args);
		FactorySettingsWidget->SetObject(FactorySettingsObject);
		
		return FactorySettingsWidget.ToSharedRef();
	}

	return SNullWidget::NullWidget;
}

FString STaggedAssetBrowserAssetFactoryWindow::GetReferencerName() const
{
	return TEXT("STaggedAssetBrowserCreateAsset");
}

void STaggedAssetBrowserAssetFactoryWindow::AddReferencedObjects(FReferenceCollector& Collector)
{
	FSlateInvalidationRoot::AddReferencedObjects(Collector);
	
	if(FactorySettingsObject)
	{
		Collector.AddReferencedObject(FactorySettingsObject);
	}
}

void STaggedAssetBrowserAssetFactoryWindow::CloseIfOtherWindowFocused(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldWidgetPath,	const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget)
{
	// If the focus path no longer contains our window, close it
	if (!NewWidgetPath.ContainsWidget(this))
	{
		Cancel();
	}
}

#undef LOCTEXT_NAMESPACE
