// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STaggedAssetBrowserContent.h"

#include "ContentBrowserModule.h"
#include "IContentBrowserDataModule.h"
#include "IContentBrowserSingleton.h"
#include "SAssetPicker.h"
#include "SlateOptMacros.h"



#define LOCTEXT_NAMESPACE "TaggedAssetBrowserContent"

void STaggedAssetBrowserContent::Construct(const FArguments& InArgs)
{	
	FAssetPickerConfig Config = InArgs._InitialConfig;
	Config.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	Config.RefreshAssetViewDelegates.Add(&RefreshAssetViewDelegate);
	Config.SyncToAssetsDelegates.Add(&SyncToAssetsDelegate);
	Config.SetFilterDelegates.Add(&SetNewFilterDelegate);
	Config.bCanShowRealTimeThumbnails = true;
	
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TSharedRef<SWidget> AssetPickerWidget = ContentBrowserModule.Get().CreateAssetPicker(Config);

	if (AssetPickerWidget->GetType() == FName(TEXT("SAssetPicker")))
	{
		AssetPickerPtr = StaticCastSharedRef<SAssetPicker>(AssetPickerWidget);
	}
	else
	{
		AssetPickerPtr = nullptr;
	}

	ChildSlot
	[
		AssetPickerWidget
	];
}

TSharedPtr<SAssetPicker> STaggedAssetBrowserContent::GetAssetPicker() const
{
	return AssetPickerPtr;
}

void STaggedAssetBrowserContent::SetARFilter(FARFilter InFilter)
{
	SetNewFilterDelegate.Execute(InFilter);
}

TArray<FAssetData> STaggedAssetBrowserContent::GetCurrentSelection() const
{
	return GetCurrentSelectionDelegate.Execute();
}

#undef LOCTEXT_NAMESPACE


