// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STaggedAssetBrowserSections.h"

#include "SlateOptMacros.h"
#include "Styling/ToolBarStyle.h"
#include "Engine/Texture2D.h"
#include "TaggedAssetBrowserFilters/TaggedAssetBrowser_CommonFilters.h"

void STaggedAssetBrowserSection::Construct(const FArguments& InArgs, const UTaggedAssetBrowserSection& InSection)
{
	Section = &InSection;

	OnCheckStateChangedDelegate = InArgs._OnCheckStateChanged;
	IsCheckedAttribute = InArgs._IsChecked;
	OnGetMenuContentDelegate = InArgs._OnGetMenuContent;
	
	const FToolBarStyle& ToolBarStyle = FAppStyle::GetWidgetStyle<FToolBarStyle>("VerticalToolBar");
	
	const FMargin IconPadding = GetLabelVisibility() == EVisibility::Visible ? ToolBarStyle.IconPaddingWithVisibleLabel : ToolBarStyle.IconPadding;

	// Use a delegate rather than static value, to account for the possibility that LabelVisiblity changes
	TAttribute<FMargin> IconPaddingAttribute = TAttribute<FMargin>::Create(
		TAttribute<FMargin>::FGetter::CreateLambda(
			[this, InitialPadding = IconPadding]() -> FMargin
			{
				FMargin IconPaddingValue = InitialPadding;

				// Icon Padding may use a bottom value appropriate for label separation, rather than for the button bounds,
				// so if the label is empty, we instead use the top padding which will be more appropriate for the button bounds.
				if (GetLabelText().IsEmpty())
				{
					IconPaddingValue.Bottom = InitialPadding.Top;
				}

				return IconPaddingValue;
			}));
	
	ChildSlot
	[
		SNew(SCheckBox)
		// Use the tool bar style for this check box
		.Style(&ToolBarStyle.ToggleButton)
		.CheckBoxContentUsesAutoWidth(true)
		.IsFocusable(false)
		.ToolTipText(Section->GetTooltip())		
		.OnCheckStateChanged(this, &STaggedAssetBrowserSection::OnCheckStateChanged)
		.IsChecked(this, &STaggedAssetBrowserSection::GetCheckState)
		.OnGetMenuContent(this, &STaggedAssetBrowserSection::OnGetMenuContent)
		.IsEnabled(this, &STaggedAssetBrowserSection::IsEnabled)
		[
			SNew(SBox)
			.WidthOverride(36.f)
			.MinDesiredHeight(46.f)
			[
				SNew(SVerticalBox)
				// Icon image
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(IconPaddingAttribute)
				.HAlign(HAlign_Center)	// Center the icon horizontally, so that large labels don't stretch out the artwork
				[
					SNew(SLayeredImage)
					.ColorAndOpacity(this, &STaggedAssetBrowserSection::GetIconForegroundColor)
					.Visibility(EVisibility::HitTestInvisible)
					.Image(this, &STaggedAssetBrowserSection::GetIconBrush)
				]
				// Label text
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(ToolBarStyle.LabelPadding)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Visibility(this, &STaggedAssetBrowserSection::GetLabelVisibility)
					.Text(this, &STaggedAssetBrowserSection::GetLabelText)
					.TextStyle(&ToolBarStyle.LabelStyle)
					.Justification(ETextJustify::Center)
					.AutoWrapText(true)
					.WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
					.Margin(FMargin(0.f, 4.f))
				]
			]
		]
	];
}

void STaggedAssetBrowserSection::OnCheckStateChanged(ECheckBoxState CheckBoxState)
{
	OnCheckStateChangedDelegate.Execute(CheckBoxState);
}

ECheckBoxState STaggedAssetBrowserSection::GetCheckState() const
{
	return IsCheckedAttribute.Get();
}

TSharedRef<SWidget> STaggedAssetBrowserSection::OnGetMenuContent()
{
	if (OnGetMenuContentDelegate.IsBound())
	{
		return OnGetMenuContentDelegate.Execute();
	}
	
	return SNullWidget::NullWidget;
}

FSlateColor STaggedAssetBrowserSection::GetIconForegroundColor() const
{
	// If any brush has a tint, don't assume it should be subdued
	const FSlateBrush* Brush = GetIconBrush();
	if (Brush && Brush->TintColor != FLinearColor::White)
	{
		return FLinearColor::White;
	}

	return FSlateColor::UseForeground();
}

const FSlateBrush* STaggedAssetBrowserSection::GetIconBrush() const
{
	return Section->IconData.GetImageBrush();
}

EVisibility STaggedAssetBrowserSection::GetLabelVisibility() const
{
	return Section->GetSectionName().IsNone() ? EVisibility::Hidden : EVisibility::Visible;
}

FText STaggedAssetBrowserSection::GetLabelText() const
{
	return Section->GetSectionNameAsText();
}

void STaggedAssetBrowserSections::Construct(const FArguments& InArgs, const UTaggedAssetBrowserFilterRoot& InFilterRoot)
{
	FilterRoot = &InFilterRoot;
	ActiveSection = InArgs._InitiallyActiveSection;
	OnSectionSelectedDelegate = InArgs._OnSectionSelected;
	
	ScrollBox = SNew(SScrollBox);

	ChildSlot
	[
		ScrollBox.ToSharedRef()
	];

	RebuildWidget();

	if(ActiveSection.IsValid())
	{
		OnSectionSelected(ECheckBoxState::Checked, ActiveSection.Get());
	}
}

void STaggedAssetBrowserSections::RebuildWidget()
{
	ScrollBox->ClearChildren();

	for(const UHierarchySection* Section : FilterRoot->GetSectionData())
	{
		const UTaggedAssetBrowserSection* TaggedAssetBrowserSection = CastChecked<UTaggedAssetBrowserSection>(Section);

		ScrollBox->AddSlot()
		[
			SNew(STaggedAssetBrowserSection, *TaggedAssetBrowserSection)
			.OnCheckStateChanged(this, &STaggedAssetBrowserSections::OnSectionSelected, TaggedAssetBrowserSection)
			.IsChecked(this, &STaggedAssetBrowserSections::IsSectionActive, TaggedAssetBrowserSection)
		];
	}
}

void STaggedAssetBrowserSections::OnSectionSelected(ECheckBoxState CheckBoxState, const UTaggedAssetBrowserSection* InSection)
{
	// We don't support deselecting sections
	ActiveSection = InSection;
	OnSectionSelectedDelegate.ExecuteIfBound(ActiveSection.Get());
}

ECheckBoxState STaggedAssetBrowserSections::IsSectionActive(const UTaggedAssetBrowserSection* InSection) const
{
	return ActiveSection == InSection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}



