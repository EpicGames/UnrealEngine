// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SUserAssetTagBrowserSelectedAssetDetails.h"

#include "UserAssetTagEditorUtilities.h"
#include "SlateOptMacros.h"
#include "TaggedAssetBrowserEditorStyle.h"
#include "Styling/SlateIconFinder.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "SUserSelectedAssetPreview"

TMap<UClass*, FAssetDetailsDisplayInfo> FTaggedAssetBrowserDetailsDisplayDatabase::Data;



void FTaggedAssetBrowserDetailsDisplayDatabase::RegisterClass(UClass* Class, FAssetDetailsDisplayInfo ClassInfo)
{
	Data.Add(Class, ClassInfo);
}

void SUserAssetTag::Construct(const FArguments& InArgs, const FName& InUserAssetTag)
{
	UserAssetTag = InUserAssetTag;
	OnAssetTagActivated = InArgs._OnAssetTagActivated;
	OnAssetTagActivatedTooltip = InArgs._OnAssetTagActivatedTooltip;
	
	TOptional<FText> Description;
	TOptional<FLinearColor> Color;
	
	if(OnAssetTagActivated.IsBound() && OnAssetTagActivatedTooltip.IsSet())
	{
		FText TooltipText = FText::FormatOrdered(FText::AsCultureInvariant("{0}{1}"), Description.Get(FText::GetEmpty()),
			OnAssetTagActivatedTooltip.GetValue());

		SetToolTipText(TooltipText);
	}
	else
	{
		SetToolTipText(Description.Get(FText::GetEmpty()));
	}

	TSharedPtr<SWidget> ContentWidget = nullptr;

	TSharedRef<STextBlock> DisplayNameWidget = SNew(STextBlock)
		.Text(FText::FromName(UserAssetTag))
		.TextStyle(&FTaggedAssetBrowserEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("TaggedAssetBrowser.AssetTag.Text"));

	if(OnAssetTagActivated.IsBound())
	{
		ContentWidget = SNew(SButton)
		.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
		.OnClicked(this, &SUserAssetTag::OnClicked)
		[
			DisplayNameWidget
		];
	}
	else
	{
		ContentWidget = DisplayNameWidget;
	}
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FTaggedAssetBrowserEditorStyle::Get().GetBrush("TaggedAssetBrowser.AssetTag.OuterBorder"))
		.BorderBackgroundColor(Color.Get(FLinearColor::Gray))
		.Padding(1.f)
		[
			SNew(SBorder)
			.BorderImage(FTaggedAssetBrowserEditorStyle::Get().GetBrush("TaggedAssetBrowser.AssetTag.InnerBorder"))
			.Padding(8.f, 2.f)
			[
				ContentWidget.ToSharedRef()
			]
		]
	];
}

FReply SUserAssetTag::OnClicked() const
{
	OnAssetTagActivated.ExecuteIfBound(UserAssetTag);
	return FReply::Handled();
}

void SUserAssetTagRow::Construct(const FArguments& InArgs, const FAssetData& Asset)
{
	using namespace UE::UserAssetTags;
	
	TSharedRef<SWrapBox> AssetTagRow = SNew(SWrapBox).UseAllottedSize(true);
	
	for(const FName& UserAssetTag : UE::UserAssetTags::GetUserAssetTagsForAssetData(Asset))
	{			
		AssetTagRow->AddSlot()
		.Padding(2.f)
		[
			SNew(SUserAssetTag, UserAssetTag)
			.OnAssetTagActivated(InArgs._OnAssetTagActivated)
			.OnAssetTagActivatedTooltip(InArgs._OnAssetTagActivatedTooltip)
		];
	}

	SetVisibility(AssetTagRow->GetChildren()->Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed);
	
	ChildSlot
	[
		AssetTagRow
	];
}

void SUserAssetTagBrowserSelectedAssetDetails::Construct(const FArguments& InArgs, const FAssetData& Asset)
{
	AssetData = Asset;
	
	ShowThumbnailSlot = InArgs._ShowThumbnailSlotWidget;
	OnGenerateThumbnailReplacementWidgetDelegate = InArgs._OnGenerateThumbnailReplacementWidget;
	OnAssetTagActivated = InArgs._OnAssetTagActivated;
	OnAssetTagActivatedTooltip = InArgs._OnAssetTagActivatedTooltip;
	
	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(200.f)
		.MaxDesiredWidth(450.f)
		.Padding(11.5f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(0.f, 0.f, 0.f, 22.f)
			[
				SNew(SBox)
				.WidthOverride(256.f)
				.HeightOverride(192.f)
				.Visibility(ShowThumbnailSlot)
				[
					OnGenerateThumbnailReplacementWidgetDelegate.IsBound() ? OnGenerateThumbnailReplacementWidgetDelegate.Execute(Asset) : CreateAssetThumbnailWidget()
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f)
			[
				CreateTitleWidget()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(150.f)
			.Padding(0.f, 6.f)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					SNew(SBox)
					.MaxDesiredWidth(InArgs._MaxDesiredDescriptionWidth)
					[
						CreateDescriptionWidget()
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 6.f)
			[
				SNew(SBox)
				.MaxDesiredWidth(InArgs._MaxDesiredPropertiesWidth)
				[
					CreateOptionalPropertiesList()
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 6.f)
			[
				CreatePathWidget()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 6.f)
			[
				CreateAssetTagRow()
			]
		]
	]; 
}

FReply SUserAssetTagBrowserSelectedAssetDetails::CopyAssetPathToClipboard() const
{
	FPlatformApplicationMisc::ClipboardCopy(*AssetData.PackagePath.ToString());
	return FReply::Handled();
}

TSharedRef<SWidget> SUserAssetTagBrowserSelectedAssetDetails::CreateAssetThumbnailWidget()
{
	CurrentAssetThumbnail = MakeShared<FAssetThumbnail>(AssetData, 256.f, 256.f, UThumbnailManager::Get().GetSharedThumbnailPool());
	FAssetThumbnailConfig Config;
	Config.bAllowRealTimeOnHovered = false;
	
	return SNew(SScaleBox)
		.Stretch(EStretch::ScaleToFill)
		[
			CurrentAssetThumbnail->MakeThumbnailWidget(Config)
		];
}

TSharedRef<SWidget> SUserAssetTagBrowserSelectedAssetDetails::CreateTitleWidget()
{
	return SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	.HAlign(HAlign_Left)
	.Padding(2.f)
	[
		SNew(STextBlock)
		.Text(FText::FromName(AssetData.AssetName))
		.TextStyle(&FTaggedAssetBrowserEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("TaggedAssetBrowser.AssetTitle"))
	]
	+ SVerticalBox::Slot()
	.AutoHeight()
	.HAlign(HAlign_Left)
	.Padding(2.f)
	[
		CreateTypeWidget()
	];
}

TSharedRef<SWidget> SUserAssetTagBrowserSelectedAssetDetails::CreateTypeWidget()
{
	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(2.f)
	[
		SNew(SImage)
		.Image(FSlateIconFinder::FindIconForClass(AssetData.GetClass()).GetIcon())
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(STextBlock)
		.Text(AssetData.GetClass()->GetDisplayNameText())
		.TextStyle(&FTaggedAssetBrowserEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("TaggedAssetBrowser.AssetType"))
	];
}

TSharedRef<SWidget> SUserAssetTagBrowserSelectedAssetDetails::CreatePathWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Path", "Path"))
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
		]
		+ SHorizontalBox::Slot()
		.Padding(5.f, 0.f)
		[
			SNew(SSpacer)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(FText::FromName(AssetData.PackagePath))
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
			.AutoWrapText(true)
		];
}

TSharedRef<SWidget> SUserAssetTagBrowserSelectedAssetDetails::CreateDescriptionWidget()
{
	if(FTaggedAssetBrowserDetailsDisplayDatabase::Data.Contains(AssetData.GetClass()) && FTaggedAssetBrowserDetailsDisplayDatabase::Data[AssetData.GetClass()].GetDescriptionDelegate.IsBound())
	{
		FText Description = FTaggedAssetBrowserDetailsDisplayDatabase::Data[AssetData.GetClass()].GetDescriptionDelegate.Execute(AssetData);
		return SNew(STextBlock)
			.AutoWrapText(true)
			.Text(Description)
			.Visibility(Description.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible);
	}

	return SNew(SBox).Visibility(EVisibility::Collapsed);
}

TSharedRef<SWidget> SUserAssetTagBrowserSelectedAssetDetails::CreateOptionalPropertiesList()
{
	if(FTaggedAssetBrowserDetailsDisplayDatabase::Data.Contains(AssetData.GetClass()))
	{
		TSharedRef<SVerticalBox> Result = SNew(SVerticalBox);
		
		for(const FDisplayedPropertyData& DisplayedProperty : FTaggedAssetBrowserDetailsDisplayDatabase::Data[AssetData.GetClass()].DisplayedProperties)
		{			
			if(DisplayedProperty.ShouldDisplayPropertyDelegate.IsBound())
			{
				if(DisplayedProperty.ShouldDisplayPropertyDelegate.Execute(AssetData) == false)
				{
					continue;
				}
			}

			TSharedRef<SHorizontalBox> DisplayedPropertyBox = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					DisplayedProperty.NameWidgetDelegate.Execute(AssetData)
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SBox)
					.MinDesiredWidth(20.f)
					[
						SNew(SSpacer)
					]
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.FillContentWidth(2.f)
				[
					DisplayedProperty.ValueWidgetDelegate.Execute(AssetData)
				];
			
			Result->AddSlot()
			.AutoHeight()
			.Padding(10.f, 3.f)
			[
				DisplayedPropertyBox
			];

			Result->AddSlot()
			.AutoHeight()
			.Padding(10.f, 3.f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Horizontal)
				.SeparatorImage(FTaggedAssetBrowserEditorStyle::Get().GetBrush("TaggedAssetBrowser.PropertySeparator"))
				.Thickness(1.f)
			];
		}

		if(Result->NumSlots() == 0)
		{
			return SNew(SBox).Visibility(EVisibility::Collapsed);
		}
		
		return Result;
	}

	return SNew(SBox).Visibility(EVisibility::Collapsed);
}

TSharedRef<SWidget> SUserAssetTagBrowserSelectedAssetDetails::CreateAssetTagRow()
{
	return SNew(SUserAssetTagRow, AssetData)
		.OnAssetTagActivated(OnAssetTagActivated)
		.OnAssetTagActivatedTooltip(OnAssetTagActivatedTooltip);
}	



#undef LOCTEXT_NAMESPACE
