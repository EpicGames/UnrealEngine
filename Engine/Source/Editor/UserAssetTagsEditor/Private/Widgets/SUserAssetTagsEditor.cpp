// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SUserAssetTagsEditor.h"

#include "ClassIconFinder.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SlateOptMacros.h"
#include "SPositiveActionButton.h"
#include "UserAssetTagEditorMenuContexts.h"
#include "UserAssetTagEditorUtilities.h"
#include "UserAssetTagMenuHelpers.h"
#include "UserAssetTagProvider.h"
#include "Config/UserAssetTagsEditorConfig.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "UserAssetTags"

TArray<const UUserAssetTagProvider*> SUserAssetTagsEditor::CachedProviderCDOs;

void SUserAssetTagsEditor::Construct(const FArguments& InArgs)
{
	ThisContext.Reset(NewObject<UUserAssetTagEditorContext>());
	
	ThisContext->UserAssetTagsEditor = SharedThis(this);
	SelectedAssetList = CreateSelectedAssetList();
	TagSuggestionList = CreateSuggestedTagList();
	OwnedTagsList = CreateOwnedTagsList();
	ToolbarWidget = CreateToolbar();

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	ContentBrowserModule.GetOnAssetSelectionChanged().AddSP(this, &SUserAssetTagsEditor::HandleAssetSelectionChanged);
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(15.f, 10.f, 15.f, 4.f)
		.HAlign(HAlign_Right)
		[
			ToolbarWidget.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		[			
			SNew(SSplitter)
			.PhysicalSplitterHandleSize(2.f)
			.Orientation(Orient_Horizontal)
			+ SSplitter::Slot()
			.Value(.35f)
			[
				SNew(SSplitter)
				.PhysicalSplitterHandleSize(2.f)
				.Orientation(Orient_Vertical)
				+ SSplitter::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
					.BorderBackgroundColor(FStyleColors::Recessed)
					[
						SelectedAssetList.ToSharedRef()
					]
				]
				+ SSplitter::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
						.BorderBackgroundColor(FStyleColors::Recessed)
						[
							OwnedTagsList.ToSharedRef()
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.Padding(5.f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							[
								SAssignNew(AddTagTextBox, SEditableTextBox)
								.HintText(LOCTEXT("AddNewTagHintText", "Add new tag"))
								.OnTextCommitted(this, &SUserAssetTagsEditor::OnCommitNewTag)
								.ClearKeyboardFocusOnCommit(false)
								.SelectAllTextOnCommit(true)
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4.f, 0.f)
							[
								SNew(SPositiveActionButton)
								.Text(LOCTEXT("AddNewTagButtonLabel", "Add"))
								.OnClicked(this, &SUserAssetTagsEditor::OnAddTagButtonClicked)
								.IsEnabled(this, &SUserAssetTagsEditor::IsAddTagButtonEnabled)
							]
						]
					]
				]
			]
			+ SSplitter::Slot()
			.Value(.65f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				.BorderBackgroundColor(FStyleColors::Recessed)
				[
					TagSuggestionList.ToSharedRef()
				]
			]
		]
	];

	RefreshDataAndMenus();

	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double CurrentTime, float DeltaTime)
	{
		FSlateApplication::Get().SetKeyboardFocus(AddTagTextBox, EFocusCause::SetDirectly);
		return EActiveTimerReturnType::Stop;
	}));
}

SUserAssetTagsEditor::~SUserAssetTagsEditor()
{
	if(FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		ContentBrowserModule->GetOnAssetSelectionChanged().RemoveAll(this);	
	}
}

TSharedPtr<SWidget> SUserAssetTagsEditor::CreateSelectedAssetList()
{
	using namespace UE::UserAssetTags::Menus;
	
	static FName MenuName = "UserAssetTags.Editor.SelectedAssets";
	if(UToolMenus::Get()->IsMenuRegistered(MenuName) == false)
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);
		ToolMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* DynamicMenu)
		{
			UUserAssetTagEditorContext* ContextObject = DynamicMenu->Context.FindContext<UUserAssetTagEditorContext>();
			if(!ensure(ContextObject))
			{
				return;
			}
			
			FToolMenuSection& OwnedTagsSection = DynamicMenu->FindOrAddSection(FName(TEXT("SelectedAssets")), LOCTEXT("SelectedAssetsLabel", "Selected Assets"));
				
			FMultipleAssetsTagInfo TagInfo = GatherInfoAboutTags(ContextObject->UserAssetTagsEditor.Pin()->GetSelectedAssets());
			
			for(const FAssetData& SelectedAsset : ContextObject->UserAssetTagsEditor.Pin()->GetSelectedAssets())
			{				
				FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(SelectedAsset.AssetName, FToolUIActionChoice(), ContextObject->UserAssetTagsEditor.Pin()->GenerateRowContent_SelectedAsset(SelectedAsset, ContextObject));
				Entry.UserInterfaceActionType = EUserInterfaceActionType::None;
				OwnedTagsSection.AddEntry(Entry);
			}

			// If we have no actual content, we still add an empty entry to make sure the section shows up
			if(OwnedTagsSection.Blocks.Num() == 0)
			{
				OwnedTagsSection.AddEntry(FToolMenuEntry::InitWidget("EmptyEntry", SNullWidget::NullWidget, FText::GetEmpty(), false, false));
			}
		}));
	}
	
	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(ThisContext.Get()));
}

TSharedPtr<SWidget> SUserAssetTagsEditor::CreateSuggestedTagList()
{
	using namespace UE::UserAssetTags::Menus;
	
	FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

	static FName MenuName = "UserAssetTags.Editor.SuggestedTags";
	if(UToolMenus::Get()->IsMenuRegistered(MenuName) == false)
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);
		ToolMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* DynamicMenu)
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			UUserAssetTagEditorContext* ContextObject = DynamicMenu->Context.FindContext<UUserAssetTagEditorContext>();
			if(!ensure(ContextObject))
			{
				return;
			}
			
			TMap<FName, FUserAssetTagInfo> AllSuggestedUserAssetTags;
			TMap<const UUserAssetTagProvider*, TSet<FName>> ProviderTagMap;
			for(const UUserAssetTagProvider* UserAssetTagProvider : ContextObject->UserAssetTagsEditor.Pin()->GetEnabledProviderCDOs(ContextObject))
			{
				TSet<FName> TagsFromProvider = UserAssetTagProvider->GetSuggestedUserAssetTags(ContextObject);
				ProviderTagMap.Add(UserAssetTagProvider, TagsFromProvider);
				
				for(const FName& UserAssetTag : TagsFromProvider)
				{
					FUserAssetTagInfo& TagInfo = AllSuggestedUserAssetTags.FindOrAdd(UserAssetTag);
					TagInfo.Sources.Add(UserAssetTagProvider);
				}
			}

			for(const auto& ProviderTagsPair : ProviderTagMap)
			{
				EUserAssetTagProviderMenuType MenuType = GetViewOptions_ProviderClassMenuType(ProviderTagsPair.Key->GetClass());
				
				FName ProviderClassName = ProviderTagsPair.Key->GetClass()->GetFName();
				FText ProviderDisplayNameText = ProviderTagsPair.Key->GetDisplayNameText(ContextObject);
				FText ProviderClassTooltip = ProviderTagsPair.Key->GetToolTipText(ContextObject);

				if(MenuType == EUserAssetTagProviderMenuType::SubMenu)
				{
					if(ProviderTagsPair.Value.Num() > 0)
					{
						TArray<FName> SuggestedUserAssetTagsCopy = ProviderTagsPair.Value.Array();
						if(UUserAssetTagsEditorConfig::Get()->ShouldSortByAlphabet())
						{
							SuggestedUserAssetTagsCopy.Sort(FNameLexicalLess());
						}
						
						FNewToolMenuDelegate NewSubMenuDelegate = FNewToolMenuDelegate::CreateLambda([ContextObject, UserAssetTags = SuggestedUserAssetTagsCopy](UToolMenu* SubMenu)
						{
							for(const FName& UserAssetTag : UserAssetTags)
							{
								FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(UserAssetTag, CreateToggleTagCheckboxAction_SuggestedTag(UserAssetTag), ContextObject->UserAssetTagsEditor.Pin()->GenerateRowContent_SuggestedTag(UserAssetTag, ContextObject));
								Entry.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;
								SubMenu->AddMenuEntry(NAME_None, Entry);	
							}
						});
						FToolMenuEntry ProviderSubMenuEntry = FToolMenuEntry::InitSubMenu(ProviderClassName, ProviderDisplayNameText, FText::GetEmpty(), NewSubMenuDelegate);
						DynamicMenu->AddMenuEntry(NAME_None, ProviderSubMenuEntry);
					}
				}
				else if(MenuType == EUserAssetTagProviderMenuType::Section)
				{
					if(ProviderTagsPair.Value.Num() > 0)
					{
						FToolMenuSection& Section = DynamicMenu->FindOrAddSection(ProviderClassName, ProviderDisplayNameText);

						TArray<FName> SuggestedUserAssetTagsCopy = ProviderTagsPair.Value.Array();
						if(UUserAssetTagsEditorConfig::Get()->ShouldSortByAlphabet())
						{
							SuggestedUserAssetTagsCopy.Sort(FNameLexicalLess());
						}
						
						for(const FName& UserAssetTag : SuggestedUserAssetTagsCopy)
						{
							FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(UserAssetTag, CreateToggleTagCheckboxAction_SuggestedTag(UserAssetTag), ContextObject->UserAssetTagsEditor.Pin()->GenerateRowContent_SuggestedTag(UserAssetTag, ContextObject));
							Entry.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;
							Section.AddEntry(Entry);
						}
					}
				}
			}

			// If there is no section, that means we have no valid provider; we add an empty "Suggested Tags" section to let the user know what is supposed to show up
			if(DynamicMenu->Sections.Num() == 0)
			{
				FToolMenuSection& Section = DynamicMenu->AddSection(NAME_None, LOCTEXT("SuggestedTagsEmptyLabel", "Suggested Tags"));
				Section.AddEntry(FToolMenuEntry::InitWidget("EmptyEntry", SNullWidget::NullWidget, FText::GetEmpty(), false, false));
			}
		}));
	}
	
	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(ThisContext.Get()));
}

TSharedPtr<SWidget> SUserAssetTagsEditor::CreateOwnedTagsList()
{
	using namespace UE::UserAssetTags::Menus;
	
	static FName MenuName = "UserAssetTags.Editor.OwnedTags";
	if(UToolMenus::Get()->IsMenuRegistered(MenuName) == false)
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);
		ToolMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* DynamicMenu)
		{
			UUserAssetTagEditorContext* ContextObject = DynamicMenu->Context.FindContext<UUserAssetTagEditorContext>();
			if(!ensure(ContextObject))
			{
				return;
			}
			
			FToolMenuSection& OwnedTagsSection = DynamicMenu->FindOrAddSection(FName(TEXT("OwnedTags")), LOCTEXT("OwnedTagsLabel", "Owned Tags"));
			
			FMultipleAssetsTagInfo TagInfo = GatherInfoAboutTags(ContextObject->UserAssetTagsEditor.Pin()->GetSelectedAssets());
			
			for(const auto& TagPairInfo : TagInfo.TagInfos)
			{
				FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(TagPairInfo.Key, CreateToggleTagCheckboxAction_OwnedTag(TagPairInfo.Key), ContextObject->UserAssetTagsEditor.Pin()->GenerateRowContent_OwnedTag(TagPairInfo.Key, ContextObject));
				ECheckBoxState CheckBoxState = DetermineTagUIStatus(TagPairInfo.Key, ContextObject->UserAssetTagsEditor.Pin()->GetSelectedAssets());
				// We only want the checkbox to show up when there the tag status is undetermined (aka some selected assets have the tag, some don't).
				// The main interface to delete tags is the delete button, but to offer consistent UI with the suggested tags we also make use of a checkbox
				// This way you can either 'fix' the tag status by clicking on the checkbox (tagging all assets), or click on the delete button to delete the tags specifically
				if(CheckBoxState == ECheckBoxState::Undetermined)
				{
					Entry.ToolTip = LOCTEXT("UndeterminedTagStatusTooltip", "Not all assets have this tag. Clicking this will apply the tag to all assets.");
					Entry.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;
				}
				// If we don't display the checkbox, remove the indent and make the row click itself do nothing
				else
				{
					Entry.UserInterfaceActionType = EUserInterfaceActionType::None;
					Entry.WidgetData.bNoIndent = true;
				}
				OwnedTagsSection.AddEntry(Entry);
			}

			if(OwnedTagsSection.Blocks.Num() == 0)
			{
				OwnedTagsSection.AddEntry(FToolMenuEntry::InitWidget("EmptyEntry", SNullWidget::NullWidget, FText::GetEmpty(), false, false));
			}
		}));
	}
	
	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(ThisContext.Get()));
}

TSharedPtr<SWidget> SUserAssetTagsEditor::CreateToolbar()
{
	using namespace UE::UserAssetTags::Menus;
	
	static FName MenuName = "UserAssetTags.Editor.Toolbar";
	if(UToolMenus::Get()->IsMenuRegistered(MenuName) == false)
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);
		ToolMenu->MenuType = EMultiBoxType::ToolBar;

		ToolMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* DynamicMenu)
		{
			UUserAssetTagEditorContext* ContextObject = DynamicMenu->Context.FindContext<UUserAssetTagEditorContext>();
			if(!ensure(ContextObject))
			{
				return;
			}

			// Extensions
			for(const UUserAssetTagProvider* UserAssetTagProvider : ContextObject->UserAssetTagsEditor.Pin()->GetEnabledProviderCDOs(ContextObject))
			{
				UserAssetTagProvider->AddToolbarMenuEntries(DynamicMenu, ContextObject);
			}
				
			DynamicMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitSeparator("Separator"));

			// Documentation
			{
				TSharedRef<SWidget> DocumentationWidget = SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(GetDocumentationButtonTooltipText())
				.OnClicked_Static(&SUserAssetTagsEditor::OnDocumentationRequested)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Documentation"))
				];

				// @TODO (ME): Readd documentation when we have a link
				// FToolMenuEntry DocumentationMenuEntry = FToolMenuEntry::InitWidget("Documentation", DocumentationWidget, FText::GetEmpty(), true);
				// DynamicMenu->AddMenuEntry(NAME_None, DocumentationMenuEntry);
			}
				
			// View Options
			{
				TSharedPtr<SLayeredImage> FilterImage = SNew(SLayeredImage)
				.Image(FAppStyle::Get().GetBrush("DetailsView.ViewOptions"))
				.ColorAndOpacity(FSlateColor::UseForeground());

				FilterImage->AddLayer(TAttribute<const FSlateBrush*>(ContextObject->UserAssetTagsEditor.Pin().ToSharedRef(), &SUserAssetTagsEditor::GetViewOptionsBadgeIcon));
				
				TSharedRef<SWidget> ViewOptionsWidget = SNew(SComboButton)
				.HasDownArrow(true)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnGetMenuContent(ContextObject->UserAssetTagsEditor.Pin().ToSharedRef(), &SUserAssetTagsEditor::OnGetViewOptions)
				.ButtonContent()
				[
					FilterImage.ToSharedRef()
				];
				
				FToolMenuEntry ViewOptionsMenuEntry = FToolMenuEntry::InitWidget("ViewOptions", ViewOptionsWidget, FText::GetEmpty(), true);
				DynamicMenu->AddMenuEntry(NAME_None, ViewOptionsMenuEntry);
			}
		}));
	}
	
	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(ThisContext.Get()));
}

TArray<FAssetData> SUserAssetTagsEditor::GetCurrentContentBrowserAssetSelection()
{
	TArray<FAssetData> TmpSelectedAssets;
	
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().GetSelectedAssets(TmpSelectedAssets);

	return TmpSelectedAssets;
}

void SUserAssetTagsEditor::RefreshDataAndMenus()
{
	SelectedAssets.Empty();
	SelectedAssets = GetCurrentContentBrowserAssetSelection();
	RefreshMenus();
}

void SUserAssetTagsEditor::RefreshMenus()
{
	RefreshSelectedAssetsMenu();
	RefreshSuggestedTagMenus();
	RefreshOwnedTagMenus();
	RefreshProviderExtensionToolbarMenu();
}

void SUserAssetTagsEditor::RefreshSelectedAssetsMenu()
{
	UToolMenus::Get()->RefreshMenuWidget("UserAssetTags.Editor.SelectedAssets");
}

void SUserAssetTagsEditor::RefreshSuggestedTagMenus()
{
	UToolMenus::Get()->RefreshMenuWidget("UserAssetTags.Editor.SuggestedTags");
}

void SUserAssetTagsEditor::RefreshOwnedTagMenus()
{
	UToolMenus::Get()->RefreshMenuWidget("UserAssetTags.Editor.OwnedTags");
}

void SUserAssetTagsEditor::RefreshProviderExtensionToolbarMenu()
{
	UToolMenus::Get()->RefreshMenuWidget("UserAssetTags.Editor.Toolbar");
}

void SUserAssetTagsEditor::OnCommitNewTag(const FText& Text, ETextCommit::Type CommitType)
{
	if(CommitType == ETextCommit::Type::OnEnter && Text.IsEmptyOrWhitespace() == false)
	{		
		UE::UserAssetTags::Menus::AddTagsToAssets(FName(Text.ToString()), SelectedAssets);

		RefreshDataAndMenus();
	}
}

FReply SUserAssetTagsEditor::OnAddTagButtonClicked()
{
	if(AddTagTextBox->GetText().IsEmptyOrWhitespace())
	{
		return FReply::Handled();
	}
	
	UE::UserAssetTags::Menus::AddTagsToAssets(FName(AddTagTextBox->GetText().ToString()), SelectedAssets);
	RefreshDataAndMenus();
	return FReply::Handled();
}

bool SUserAssetTagsEditor::IsAddTagButtonEnabled() const
{
	return AddTagTextBox->GetText().IsEmptyOrWhitespace() == false && AddTagTextBox->GetText().ToString().Contains("\\") == false;
}

const FSlateBrush* SUserAssetTagsEditor::GetViewOptionsBadgeIcon() const
{
	// If the enabled cdos match don't match the default, we display a badge
	bool bHasBadge = GetAllValidDefaultEnabledProviderCDOs(ThisContext.Get()) != GetEnabledProviderCDOs(ThisContext.Get());
	return bHasBadge ? FAppStyle::Get().GetBrush("Icons.BadgeModified") : nullptr;
}

TSharedRef<SWidget> SUserAssetTagsEditor::GenerateRowContent_SelectedAsset(const FAssetData& AssetData, const UUserAssetTagEditorContext* Context)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(0.f, 2.f, 2.f, 2.f)
		[
			SNew(SBox)
			.WidthOverride(16.f)
			.HeightOverride(16.f)
			[
				SNew(SImage)
				.Image(FClassIconFinder::FindThumbnailForClass(AssetData.GetClass()))
			]
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(2.f, 2.f)
		[
			SNew(STextBlock).Text(FText::FromName(AssetData.AssetName))
		];
}

TSharedRef<SWidget> SUserAssetTagsEditor::GenerateRowContent_SuggestedTag(FName InUserAssetTag, const UUserAssetTagEditorContext* Context)
{
	using namespace UE::UserAssetTags::Menus;
	
	TSharedPtr<SHorizontalBox> ExtensionBox;
	
	TSharedRef<SWidget> TagWidget = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Text(FText::FromName(InUserAssetTag))
	]
	+ SHorizontalBox::Slot()
	[
		SNew(SSpacer)
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	[
		SAssignNew(ExtensionBox, SHorizontalBox)
	];
	
	for(const UUserAssetTagProvider* ProviderCDO : GetEnabledProviderCDOs(Context))
	{
		if(TSharedPtr<SWidget> Widget = ProviderCDO->AddAdditionalSuggestedWidgets(InUserAssetTag, Context); Widget.IsValid())
		{
			ExtensionBox->AddSlot()
			.AutoWidth()
			[
				Widget.ToSharedRef()
			];
		}
	}
	
	return TagWidget;
}

TSharedRef<SWidget> SUserAssetTagsEditor::GenerateRowContent_OwnedTag(FName InUserAssetTag, const UUserAssetTagEditorContext* Context)
{
	using namespace UE::UserAssetTags::Menus;
	
	TSharedPtr<SHorizontalBox> ExtensionBox;
	
	TSharedRef<SWidget> TagWidget = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(2.f, 3.f)
	[
		SNew(SButton)
		.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
		.OnClicked_Static(&SUserAssetTagsEditor::OnDeleteTagClicked, InUserAssetTag, Context)
		.ToolTipText(LOCTEXT("DeleteTagButtonTooltip", "Delete this tag from all selected assets"))
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Delete"))
		]
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(2.f, 3.f)
	[
		SNew(STextBlock)
		.Text(FText::FromName(InUserAssetTag))
	]
	+ SHorizontalBox::Slot()
	[
		SNew(SSpacer)
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	[
		SAssignNew(ExtensionBox, SHorizontalBox)
	];
	
	for(const UUserAssetTagProvider* ProviderCDO : GetEnabledProviderCDOs(Context))
	{
		if(TSharedPtr<SWidget> Widget = ProviderCDO->AddAdditionalOwnedTagWidgets(InUserAssetTag, Context); Widget.IsValid())
		{
			ExtensionBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				Widget.ToSharedRef()
			];
		}
	}
	
	return TagWidget;
}

TSharedRef<SWidget> SUserAssetTagsEditor::CreateMenuControlWidget(const FToolMenuContext& ToolMenuContext, const FToolMenuCustomWidgetContext& ToolMenuCustomWidgetContext, const UClass* Class)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f)
		[
			SNew(STextBlock).Text(Class->GetDefaultObject<UUserAssetTagProvider>()->GetDisplayNameText(ToolMenuContext.FindContext<UUserAssetTagEditorContext>()))
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SSegmentedControl<EUserAssetTagProviderMenuType>)
			.Value_Static(&SUserAssetTagsEditor::GetViewOptions_ProviderClassMenuType, Class)
			.OnValueChanged_Static(&SUserAssetTagsEditor::SetViewOption_ProviderClassMenuType, Class)

			+ SSegmentedControl<EUserAssetTagProviderMenuType>::Slot(EUserAssetTagProviderMenuType::Section)
			.Text(INVTEXT("Section"))

			+ SSegmentedControl<EUserAssetTagProviderMenuType>::Slot(EUserAssetTagProviderMenuType::SubMenu)
			.Text(INVTEXT("Menu"))
		];
}

TSharedRef<SWidget> SUserAssetTagsEditor::OnGetViewOptions()
{
	using namespace UE::UserAssetTags::Menus;
	
	static FName MenuName = "UserAssetTags.Editor.SuggestedTags.ViewOptions";
	if(UToolMenus::Get()->IsMenuRegistered(MenuName) == false)
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);
		ToolMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* DynamicMenu)
		{
			UUserAssetTagEditorContext* ContextObject = DynamicMenu->Context.FindContext<UUserAssetTagEditorContext>();
			if(!ensure(ContextObject))
			{
				return;
			}

			FToolMenuSection& OptionsSection = DynamicMenu->AddSection(FName("Options"), LOCTEXT("OptionsSectionName", "Options"));

			FToolUIAction SortByAlphabetAction;
			SortByAlphabetAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&SUserAssetTagsEditor::ToggleSortByAlphabet);
			SortByAlphabetAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&SUserAssetTagsEditor::GetShouldSortByAlphabet);
			FToolMenuEntry SortByAlphabetMenuEntry = FToolMenuEntry::InitMenuEntry(
				"SortByAlphabet",
				LOCTEXT("ViewOptionsLabel", "Sort by Alphabet"),
				FText::GetEmpty(),
				FSlateIcon(),
				SortByAlphabetAction, EUserInterfaceActionType::ToggleButton);
				
			OptionsSection.AddEntry(SortByAlphabetMenuEntry);
				
			FToolMenuSection& ProvidersSection = DynamicMenu->AddSection(FName("Providers"), LOCTEXT("TagProvidersLabel", "Providers"));
			
			for(const UUserAssetTagProvider* ProviderCDO : ContextObject->UserAssetTagsEditor.Pin()->GetAllProviderCDOs())
			{
				UUserAssetTagProvider::FResultWithUserFeedback Result = ProviderCDO->IsValid(ContextObject);
				const UClass* Class = ProviderCDO->GetClass();

				FText Tooltip;

				if(Result.UserFeedback.IsSet())
				{
					Tooltip = FText::FormatOrdered(INVTEXT("{0}\n\n{1}"), ProviderCDO->GetToolTipText(ContextObject), Result.UserFeedback.GetValue());	
				}
				else
				{
					Tooltip = ProviderCDO->GetToolTipText(ContextObject);
				}
				
				if(Result.bResult)
				{
					FToolUIAction Action;
					Action.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&SUserAssetTagsEditor::ToggleViewOption_ProviderClass, Class);
					Action.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&SUserAssetTagsEditor::GetViewOptions_IsProviderClassEnabled, Class);
					FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(ProviderCDO->GetFName(), Class->GetDisplayNameText(), Tooltip, FSlateIcon(), Action, EUserInterfaceActionType::ToggleButton);
					Entry.MakeCustomWidget = FNewToolMenuCustomWidget::CreateStatic(&SUserAssetTagsEditor::CreateMenuControlWidget, Class);
					ProvidersSection.AddEntry(Entry);
				}
				else
				{
					FToolUIAction Action;
					Action.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([](const FToolMenuContext&)
					{
						return ECheckBoxState::Unchecked;
					});
					Action.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&)
					{
						return false;
					});
					
					FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(ProviderCDO->GetFName(), Class->GetDisplayNameText(), Tooltip, FSlateIcon(), Action, EUserInterfaceActionType::ToggleButton); 
					ProvidersSection.AddEntry(Entry);
				}
			}
		}));
	}
	
	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(ThisContext.Get()));
}

void SUserAssetTagsEditor::ToggleSortByAlphabet(const FToolMenuContext& ToolMenuContext)
{
	UUserAssetTagsEditorConfig::Get()->ToggleSortByAlphabet();
	RefreshMenus();
}

ECheckBoxState SUserAssetTagsEditor::GetShouldSortByAlphabet(const FToolMenuContext& ToolMenuContext)
{
	return UUserAssetTagsEditorConfig::Get()->ShouldSortByAlphabet() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SUserAssetTagsEditor::ToggleViewOption_ProviderClass(const FToolMenuContext& ToolMenuContext, const UClass* ProviderClass)
{
	UUserAssetTagsEditorConfig::Get()->ToggleProviderEnabled(ProviderClass);
	RefreshMenus();
}

ECheckBoxState SUserAssetTagsEditor::GetViewOptions_IsProviderClassEnabled(const FToolMenuContext& ToolMenuContext, const UClass* ProviderClass)
{
	return UUserAssetTagsEditorConfig::Get()->IsProviderEnabled(ProviderClass) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SUserAssetTagsEditor::SetViewOption_ProviderClassMenuType(EUserAssetTagProviderMenuType InMenuType, const UClass* ProviderClass)
{
	UUserAssetTagsEditorConfig::Get()->SetProviderMenuType(ProviderClass, InMenuType);
	RefreshMenus();
}

EUserAssetTagProviderMenuType SUserAssetTagsEditor::GetViewOptions_ProviderClassMenuType( const UClass* ProviderClass)
{
	return UUserAssetTagsEditorConfig::Get()->GetProviderMenuType(ProviderClass);
}

const TArray<const UUserAssetTagProvider*>& SUserAssetTagsEditor::GetAllProviderCDOs()
{
	// We detect all provider CDOs once; it's unlikely it needs to be refreshed at all
	if(CachedProviderCDOs.IsEmpty())
	{		
		TArray<UClass*> ProviderClasses;
		GetDerivedClasses(UUserAssetTagProvider::StaticClass(), ProviderClasses);

		ProviderClasses.RemoveAll([](const UClass* Candidate)
		{
			if(Candidate->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
			{
				return true;
			}

			return false;
		});
		
		Algo::Transform(ProviderClasses, CachedProviderCDOs, [](const UClass* Candidate)
		{
			return Candidate->GetDefaultObject<UUserAssetTagProvider>();
		});
	}

	return CachedProviderCDOs;
}

TArray<const UUserAssetTagProvider*> SUserAssetTagsEditor::GetAllValidDefaultEnabledProviderCDOs(const UUserAssetTagEditorContext* Context)
{
	TArray<const UUserAssetTagProvider*> Result = GetAllProviderCDOs();

	Result.RemoveAll([Context](const UUserAssetTagProvider* Provider)
	{
		return Provider->IsEnabledByDefault() == false || Provider->IsValid(Context).bResult == false;
	});

	return Result;
}

TArray<const UUserAssetTagProvider*> SUserAssetTagsEditor::GetValidProviderCDOs(const UUserAssetTagEditorContext* Context)
{
	TArray<const UUserAssetTagProvider*> Result = GetAllProviderCDOs();

	Result.RemoveAll([Context](const UUserAssetTagProvider* Candidate)
	{
		return Candidate->IsValid(Context) == false;
	});
	
	return Result;
}

TArray<const UUserAssetTagProvider*> SUserAssetTagsEditor::GetEnabledProviderCDOs(const UUserAssetTagEditorContext* Context)
{
	TArray<const UUserAssetTagProvider*> Result = GetValidProviderCDOs(Context);

	Result.RemoveAll([](const UUserAssetTagProvider* Candidate)
	{
		if(UUserAssetTagsEditorConfig::Get()->IsProviderEnabled(Candidate->GetClass()))
		{
			return false;
		}

		return true;
	});
	
	return Result;
}

FToolUIAction SUserAssetTagsEditor::CreateToggleTagCheckboxAction_SuggestedTag(const FName& InUserAssetTag)
{
	using namespace UE::UserAssetTags::Menus;
	
	FToolUIAction Action;
	Action.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([InUserAssetTag](const FToolMenuContext& Context)
	{
		UUserAssetTagEditorContext* ContextObject = Context.FindContext<UUserAssetTagEditorContext>();
		return DetermineTagUIStatus(InUserAssetTag, ContextObject->UserAssetTagsEditor.Pin()->GetSelectedAssets());
	});
	Action.ExecuteAction = FToolMenuExecuteAction::CreateLambda([InUserAssetTag](const FToolMenuContext& Context)
	{
		UUserAssetTagEditorContext* ContextObject = Context.FindContext<UUserAssetTagEditorContext>();
		const TArray<FAssetData>& Assets = ContextObject->UserAssetTagsEditor.Pin()->GetSelectedAssets();
					
		ECheckBoxState CurrentState = DetermineTagUIStatus(InUserAssetTag, Assets);
		if(CurrentState == ECheckBoxState::Checked)
		{
			RemoveTagsFromAssets(InUserAssetTag, Assets);
		}
		else if(CurrentState == ECheckBoxState::Unchecked)
		{
			AddTagsToAssets(InUserAssetTag, Assets);
		}
		else if(CurrentState == ECheckBoxState::Undetermined)
		{
			AddTagsToAssets(InUserAssetTag, Assets);
		}

		ContextObject->UserAssetTagsEditor.Pin()->RefreshDataAndMenus();
	});

	return Action;
}

FToolUIAction SUserAssetTagsEditor::CreateToggleTagCheckboxAction_OwnedTag(const FName& InUserAssetTag)
{
	using namespace UE::UserAssetTags::Menus;
	
	FToolUIAction Action;
	Action.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([InUserAssetTag](const FToolMenuContext& Context)
	{
		UUserAssetTagEditorContext* ContextObject = Context.FindContext<UUserAssetTagEditorContext>();
		return DetermineTagUIStatus(InUserAssetTag, ContextObject->UserAssetTagsEditor.Pin()->GetSelectedAssets());
	});
	Action.ExecuteAction = FToolMenuExecuteAction::CreateLambda([InUserAssetTag](const FToolMenuContext& Context)
	{
		UUserAssetTagEditorContext* ContextObject = Context.FindContext<UUserAssetTagEditorContext>();
		const TArray<FAssetData>& Assets = ContextObject->UserAssetTagsEditor.Pin()->GetSelectedAssets();
					
		ECheckBoxState CurrentState = DetermineTagUIStatus(InUserAssetTag, Assets);
		if(CurrentState == ECheckBoxState::Checked)
		{
			RemoveTagsFromAssets(InUserAssetTag, Assets);
		}
		else if(CurrentState == ECheckBoxState::Unchecked)
		{
			AddTagsToAssets(InUserAssetTag, Assets);
		}
		else if(CurrentState == ECheckBoxState::Undetermined)
		{
			AddTagsToAssets(InUserAssetTag, Assets);
		}

		ContextObject->UserAssetTagsEditor.Pin()->RefreshDataAndMenus();
	});

	return Action;
}

FReply SUserAssetTagsEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(InKeyEvent.GetKey() == EKeys::Escape)
	{
		if(TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared()))
		{
			Window->RequestDestroyWindow();
			return FReply::Handled();
		}
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SUserAssetTagsEditor::OnDocumentationRequested()
{
	FPlatformProcess::LaunchURL(TEXT("https://dev.epicgames.com/community/learning/tutorials/RJbm/unreal-engine-niagara-data-channels-intro"), nullptr, nullptr);
	return FReply::Handled();
}

FText SUserAssetTagsEditor::GetDocumentationButtonTooltipText()
{
	return LOCTEXT("DocumentationButtonTooltipText", "Click on the button to open the documentation and learn more.\n\nUser Asset Tags allow you to add arbitrary tags to your assets.\n\nTags can be used to search for assets, present assets in an organized manner in asset wizards, or used at runtime via Asset Registry tags.\n\nOwned Tags are the tags currently present on your selected assets.\nSuggested Tags are tags you might want to tag your assets with, but you can add any tag you want.");
}

FReply SUserAssetTagsEditor::OnDeleteTagClicked(FName InUserAssetTag, const UUserAssetTagEditorContext* Context)
{
	using namespace UE::UserAssetTags::Menus;
	const TArray<FAssetData>& Assets = Context->UserAssetTagsEditor.Pin()->GetSelectedAssets();
	RemoveTagsFromAssets(InUserAssetTag, Assets);
	RefreshMenus();
	return FReply::Handled();
}

void SUserAssetTagsEditor::HandleAssetSelectionChanged(const TArray<FAssetData>& AssetData, bool bIsPrimaryBrowser)
{
	RefreshDataAndMenus();
}

#undef LOCTEXT_NAMESPACE


