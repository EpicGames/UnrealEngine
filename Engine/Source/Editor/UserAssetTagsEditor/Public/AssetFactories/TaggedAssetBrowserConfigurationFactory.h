// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "TaggedAssetBrowserConfigurationFactory.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

UCLASS(MinimalAPI)
class UTaggedAssetBrowserConfigurationFactory : public UFactory
{
	GENERATED_BODY()

	UE_API UTaggedAssetBrowserConfigurationFactory();
	
	UE_API virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

#undef UE_API
