// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "TaggedAssetBrowserConfiguration.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

class UTaggedAssetBrowserFilterBase;
class UTaggedAssetBrowserFilterRoot;

USTRUCT()
struct FTaggedAssetBrowserConfigurationDataBase
{
	GENERATED_BODY()
};

/** Data relevant for standalone assets. */
USTRUCT()
struct FTaggedAssetBrowserConfigurationData_Standalone : public FTaggedAssetBrowserConfigurationDataBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category="Configuration")
	bool bDisplayAllSection = true;

	/** True implies we should merge extensions by section */
	UPROPERTY(EditAnywhere, Category = "Configuration")
	bool bMergeExtensionsPerSection = true;
	
	/** The list of available hierarchy elements to add in the filter hierarchy for standalone configuration assets. */
	UPROPERTY(EditAnywhere, Category="Configuration", meta=(ShowDisplayNames))
	TArray<TSubclassOf<UTaggedAssetBrowserFilterBase>> StandaloneFilterClasses;
    
	/** The list of available hierarchy elements to add in the filter hierarchy and section filters for extension assets, as defined by a standalone asset. */
	UPROPERTY(EditAnywhere, Category="Configuration", meta=(ShowDisplayNames))
	TArray<TSubclassOf<UTaggedAssetBrowserFilterBase>> ExtensionFilterClasses;
};

/** Data relevant for extension assets. Currently empty. */
USTRUCT()
struct FTaggedAssetBrowserConfigurationData_Extension : public FTaggedAssetBrowserConfigurationDataBase
{
	GENERATED_BODY()
};

/** The actual configuration asset. Contains a hierarchy root that lets you add new elements and has drag & drop support via Data Hierarchy Editor. */
UCLASS(MinimalAPI)
class UTaggedAssetBrowserConfiguration : public UObject
{
	GENERATED_BODY()
public:

	UE_API UTaggedAssetBrowserConfiguration();

	/** Used for saving filter state and various menus. Should be unique. Is also used for extension identification. */
	UPROPERTY(EditAnywhere, Category="Configuration", AssetRegistrySearchable)
	FName ProfileName;
	
	/** If this asset is used as extension, the matching standalone configuration asset will dictate what elements you can add.
	 *  The ProfileName has to match the standalone asset. */
	UPROPERTY(EditAnywhere, Category="Configuration", AssetRegistrySearchable)
	bool bIsExtension = false;
	
	UPROPERTY(EditAnywhere, Category="Configuration", meta=(EditCondition="bIsExtension==false", EditConditionHides))
	FTaggedAssetBrowserConfigurationData_Standalone StandaloneData;
	
	UPROPERTY(EditAnywhere, Category="Configuration", meta=(EditCondition="bIsExtension==true", EditConditionHides))
	FTaggedAssetBrowserConfigurationData_Extension ExtensionData;
	
	UPROPERTY()
	TObjectPtr<UTaggedAssetBrowserFilterRoot> FilterRoot;

protected:
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
};

#undef UE_API
