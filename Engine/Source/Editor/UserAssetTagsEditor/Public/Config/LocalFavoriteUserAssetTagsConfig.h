// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorConfigBase.h"
#include "LocalFavoriteUserAssetTagsConfig.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

USTRUCT()
struct FPerTypeFavoriteUserAssetTags
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="User Asset Tags")
	TSet<FName> FavoriteUserAssetTags;
};
/**
 * 
 */
UCLASS(MinimalAPI, EditorConfig="FavoriteUserAssetTags")
class ULocalFavoriteUserAssetTagsConfig : public UEditorConfigBase
{
	GENERATED_BODY()

public:
	static UE_API ULocalFavoriteUserAssetTagsConfig* Get();
	static UE_API void Shutdown();
	
	TMap<FSoftClassPath, FPerTypeFavoriteUserAssetTags>& GetFavoriteUserAssetTagsMutable() { return FavoriteUserAssetTagsPerClass; }
	
	const TMap<FSoftClassPath, FPerTypeFavoriteUserAssetTags>& GetFavoriteUserAssetTags() const { return FavoriteUserAssetTagsPerClass; }

	UE_API TSet<FName> GetFavoriteUserAssetTagsForClass(const UClass* Class);
	UE_API void ToggleFavoriteUserAssetTag(const UClass* Class, FName InUserAssetTag);
private:
	UPROPERTY(meta=(EditorConfig))
	TMap<FSoftClassPath, FPerTypeFavoriteUserAssetTags> FavoriteUserAssetTagsPerClass;

	UPROPERTY(meta=(EditorConfig))
	int32 MaxRecentUserAssetTags = 10;
		
	static UE_API TStrongObjectPtr<ULocalFavoriteUserAssetTagsConfig> Instance;
};

#undef UE_API
