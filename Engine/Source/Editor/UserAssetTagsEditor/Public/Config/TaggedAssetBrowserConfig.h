// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorConfigBase.h"
#include "TaggedAssetBrowserConfig.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

USTRUCT()
struct FPerTaggedAssetBrowserSavedState
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FName> PrimaryFilterSelection;
};

UCLASS(MinimalAPI, EditorConfig="TaggedAssetBrowser")
class UTaggedAssetBrowserConfig : public UEditorConfigBase
{
	GENERATED_BODY()
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPropertyChanged, const FPropertyChangedEvent&);
		
	static UE_API UTaggedAssetBrowserConfig* Get();
	static UE_API void Shutdown();
	static UE_API bool HasInstance();
	
	FOnPropertyChanged& OnPropertyChanged() { return OnPropertyChangedDelegate; }
	
	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FPerTaggedAssetBrowserSavedState> PerTaggedAssetBrowserSettings;
	
	UPROPERTY(meta=(EditorConfig))
    bool bShowHiddenAssets = false;
    	
	UPROPERTY(meta=(EditorConfig))
	bool bShowDeprecatedAssets = false;

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;	
private:
	static UE_API TStrongObjectPtr<UTaggedAssetBrowserConfig> Instance;

	FOnPropertyChanged OnPropertyChangedDelegate;
};

#undef UE_API
