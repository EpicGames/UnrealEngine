// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorConfigBase.h"
#include "UserAssetTagProvider.h"
#include "UObject/StrongObjectPtrTemplates.h"
#include "UserAssetTagsEditorConfig.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

USTRUCT()
struct FPerUserAssetTagProviderViewOptions
{
	GENERATED_BODY()

	UPROPERTY()
	bool bEnabled = true;

	/** 0: Section; 1: SubMenu. */
	UPROPERTY()
	EUserAssetTagProviderMenuType MenuType = EUserAssetTagProviderMenuType::Section;
};

USTRUCT()
struct FUserAssetTagProviderViewOptions
{
	GENERATED_BODY()
	
	UPROPERTY()
	TMap<FName, FPerUserAssetTagProviderViewOptions> PerProviderViewOptions;
};
/**
 * 
 */
UCLASS(MinimalAPI, EditorConfig=UserAssetTags)
class UUserAssetTagsEditorConfig : public UEditorConfigBase
{
	GENERATED_BODY()

public:
	UE_API void ToggleSortByAlphabet();
	UE_API bool ShouldSortByAlphabet() const;
	
	UE_API bool IsProviderEnabled(const UClass* ProviderClass) const;
	UE_API void ToggleProviderEnabled(const UClass* ProviderClass);
	
	UE_API EUserAssetTagProviderMenuType GetProviderMenuType(const UClass* ProviderClass) const;
	UE_API void SetProviderMenuType(const UClass* ProviderClass, EUserAssetTagProviderMenuType InMenuType);

	static UE_API UUserAssetTagsEditorConfig* Get();
	static UE_API void Shutdown();
private:
	UPROPERTY(meta=(EditorConfig))
	bool bSortByAlphabet = false;
	
	UPROPERTY(meta=(EditorConfig))
	FUserAssetTagProviderViewOptions ProviderViewOptions;
private:
    static UE_API TStrongObjectPtr<UUserAssetTagsEditorConfig> Instance;
};

#undef UE_API
