// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataHierarchyViewModelBase.h"
#include "TaggedAssetBrowserConfigurationHierarchyViewModel.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

class UTaggedAssetBrowserSection;
class UTaggedAssetBrowserFilterBase;
class UTaggedAssetBrowserFilter_All;
class UTaggedAssetBrowserFilter_UserAssetTag;
class UTaggedAssetBrowserFilter_UserAssetTagCollection;
class UTaggedAssetBrowserConfiguration;

/**
 * The view model for Data Hierarchy Editor integration to author filter views for a Tagged Asset Browser.
 */
UCLASS(MinimalAPI)
class UTaggedAssetBrowserConfigurationHierarchyViewModel : public UDataHierarchyViewModelBase
{
	GENERATED_BODY()

public:
	UE_API void Initialize(UTaggedAssetBrowserConfiguration& InAsset);
	UTaggedAssetBrowserConfiguration* GetConfigurationAsset() const { return ConfigurationAsset.Get(); }
	
	UE_API virtual UHierarchyRoot* GetHierarchyRoot() const override;
	UE_API virtual TSharedPtr<FHierarchyElementViewModel> CreateCustomViewModelForElement(UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> Parent) override;

	/** We don't need a sources panel or root as we only add new elements. */
	virtual bool SupportsSourcePanel() const override { return false; }

	/** We specify our custom section class */
	UE_API virtual TSubclassOf<UHierarchySection> GetSectionDataClass() const override;

	/** We disables categories */
	virtual TSubclassOf<UHierarchyCategory> GetCategoryDataClass() const override { return nullptr; }

	/** The types to add in the UI are specified here. We differentiate between standalone and extension elements. */
	UE_API virtual TArray<TSubclassOf<UHierarchyElement>> GetAdditionalTypesToAddInUi() const override;

	/** If we select nothing, we select the root. The custom root view model points at the asset for editing in the details panel. */
	virtual bool SelectRootInsteadOfNone() const override { return true; }
private:
	UE_API TArray<TSubclassOf<UTaggedAssetBrowserFilterBase>> GetAdditionalTypesToAddInUi_Standalone() const;
	UE_API TArray<TSubclassOf<UTaggedAssetBrowserFilterBase>> GetAdditionalTypesToAddInUi_Extension() const;

	UE_API void EmptyCache(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

protected:
	UE_API virtual void InitializeInternal() override;
	UE_API virtual void FinalizeInternal() override;
protected:
	TWeakObjectPtr<UTaggedAssetBrowserConfiguration> ConfigurationAsset;
	mutable TOptional<TArray<TSubclassOf<UHierarchyElement>>> AdditionalTypesCache;
};

namespace UE::UserAssetTags::ViewModels
{
	struct FFilterViewModel_Root : public FHierarchyRootViewModel
	{
		FFilterViewModel_Root(UHierarchyRoot* Root, TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel, bool bInIsForHierarchy)
			: FHierarchyRootViewModel(Root, ViewModel, bInIsForHierarchy)	{}
	
	private:
		/** Unlike the default implementation, our root can accept items even when another section is active. */
		virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone) override;
		/** If the root gets selected, we select the actual asset instead since it houses properties we want to be able to edit. */
		virtual UObject* GetObjectForEditing() override;
	};

	/** We have a custom section view model that supports associating random elements to sections. */
	struct FFilterViewModel_Section : public FHierarchySectionViewModel
	{
		FFilterViewModel_Section(UTaggedAssetBrowserSection* InSection, TSharedRef<FHierarchyRootViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel);

		virtual const FSlateBrush* GetSectionImageBrush() const override;
		virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType) override;
	};

	/** The base view model for all our filters. */
	struct FFilterViewModelBase : public FHierarchyItemViewModel
	{
		FFilterViewModelBase(UTaggedAssetBrowserFilterBase* InFilter, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel);
	};
	
	struct FFilterViewModel_All : public FFilterViewModelBase
	{
		FFilterViewModel_All(UTaggedAssetBrowserFilter_All* InFilter, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel);
		
		virtual FResultWithUserFeedback CanHaveChildren() const override { return true; }
	};

	struct FFilterViewModel_UserAssetTag : public FFilterViewModelBase
	{
		FFilterViewModel_UserAssetTag(UTaggedAssetBrowserFilter_UserAssetTag* InFilter, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel);
		
		virtual FResultWithUserFeedback CanHaveChildren() const override { return true; }
		virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType) override;
		virtual bool CanRenameInternal() override { return true; }
		virtual void RenameInternal(FName NewName) override;
	};

	struct FFilterViewModel_UserAssetTagCollection : public FFilterViewModelBase
	{
		FFilterViewModel_UserAssetTagCollection(UTaggedAssetBrowserFilter_UserAssetTagCollection* InFilter, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel);

		virtual FResultWithUserFeedback CanHaveChildren() const override { return true; }
		virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType) override;
		virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel>, EItemDropZone ItemDropZone) override;

		virtual bool CanRenameInternal() override { return true; }
		virtual void RenameInternal(FName NewName) override;
	};
}

#undef UE_API
