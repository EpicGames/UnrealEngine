// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/** Manages the style which provides resources for niagara editor widgets. */
class FTaggedAssetBrowserEditorStyle : public FSlateStyleSet
{
public:
	static void Register();
	static void Unregister();
	static void Shutdown();

	/** reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set for niagara editor widgets */
	static const FTaggedAssetBrowserEditorStyle& Get();

	static void ReinitializeStyle();

private:	
	FTaggedAssetBrowserEditorStyle();

	void InitStyle();
	
	static TSharedPtr<FTaggedAssetBrowserEditorStyle> TaggedAssetBrowserEditorStyle;
};
