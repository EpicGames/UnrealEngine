// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TaggedAssetBrowserMenuFilters.h"
#include "UObject/Object.h"
#include "MRUFavoritesList.h"
#include "TaggedAssetBrowser_CommonFilters.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

class UTaggedAssetBrowserSection;

UCLASS()
class UTaggedAssetBrowserFilterRoot : public UHierarchyRoot
{
	GENERATED_BODY()
};

USTRUCT()
struct FTaggedAssetBrowserSectionIconData
{
	GENERATED_BODY()

	/** If true, will use a texture. */
	UPROPERTY(EditAnywhere, Category="Icon")
	bool bUseTextureForIcon = false;

	/** The actual name of the icon to use within the styleset. */
	UPROPERTY(EditAnywhere, Category="Icon", meta=(EditCondition="bUseTextureForIcon==false"))
	FName StyleName;

	/** The icon to display for this section. */
	UPROPERTY(EditAnywhere, Category = "Icon", meta=(EditCondition="bUseTextureForIcon==true"))
	TObjectPtr<UTexture2D> Icon;

	const FSlateBrush* GetImageBrush() const;
private:
	mutable FSlateBrush TextureBrush;
};

/** The section class for the tagged asset browser. Can contain additional filters.*/
UCLASS(MinimalAPI)
class UTaggedAssetBrowserSection : public UHierarchySection
{
	GENERATED_BODY()

public:
	UTaggedAssetBrowserSection() = default;

	UPROPERTY(EditAnywhere, Category = "Section", meta=(ShowOnlyInnerProperties))
	FTaggedAssetBrowserSectionIconData IconData;

	/** True implies this section should support merging  */
	UPROPERTY(EditAnywhere, Category = "Section")
	bool bAllowSectionMerge = true;

	/** A list of filters that can be specified optionally. This way, a section can display its associated elements, OR additionally filter assets by itself. */
	UPROPERTY(EditAnywhere, Category = "Section", Instanced)
	TArray<TObjectPtr<UTaggedAssetBrowserFilterBase>> Filters;
};

/** Show all assets. */
UCLASS(MinimalAPI, DisplayName="All")
class UTaggedAssetBrowserFilter_All : public UTaggedAssetBrowserFilterBase
{
	GENERATED_BODY()
};

/** A filter that returns assets that have the specified User Asset Tag. Nested User Asset Tag filters will be combined. */
UCLASS(MinimalAPI, DisplayName="User Asset Tag")
class UTaggedAssetBrowserFilter_UserAssetTag : public UTaggedAssetBrowserFilterBase
{
	GENERATED_BODY()
public:
	UE_API virtual FString ToString() const override;
	virtual TOptional<FString> GetInstanceIdentifier() const override { return ToString(); }
	UE_API virtual FSlateIcon GetIcon() const override;
	UE_API virtual void ModifyARFilterInternal(FARFilter& Filter) const override;
	UE_API virtual bool ShouldFilterAssetInternal(const FAssetData& InAssetData) const override;

	UE_API void SetUserAssetTag(FName InUserAssetTag);
	UE_API bool DoesAssetHaveTag(const FAssetData& AssetCandidate) const;

protected:
	UPROPERTY(EditAnywhere, Category="Filter")
	FName UserAssetTag;
};

/** A collection of multiple User Asset Tags. Will show all assets that own at least one of the contained tags. */
UCLASS(MinimalAPI, DisplayName="User Asset Tag Collection")
class UTaggedAssetBrowserFilter_UserAssetTagCollection : public UTaggedAssetBrowserFilterBase
{
	GENERATED_BODY()

public:

	UE_API virtual FString ToString() const override;
	virtual TOptional<FString> GetInstanceIdentifier() const override { return ToString(); }
	UE_API virtual FText GetTooltip() const override;
	UE_API virtual FSlateIcon GetIcon() const override;
	UE_API virtual void CreateAdditionalWidgets(TSharedPtr<SHorizontalBox> ExtensionBox) override;

	UE_API virtual void ModifyARFilterInternal(FARFilter& Filter) const override;
	UE_API virtual bool ShouldFilterAssetInternal(const FAssetData& InAssetData) const override;
	
	UE_API void SetCollectionName(FName InName);
	
protected:
	UPROPERTY(EditAnywhere, Category="Filter")
	FName Name;
	
	UPROPERTY(EditAnywhere, Category="Filter", meta=(MultiLine))
	FText Description;
	
private:
	UE_API FText GetContainedChildrenNumberText() const;
	UE_API EVisibility GetContainedChildrenNumberTextVisibility() const;
	UE_API FText GetContainedChildrenNumberTooltip() const;
};

/** Filter for recently used assets. */
UCLASS(MinimalAPI, DisplayName="Recent")
class UTaggedAssetBrowserFilter_Recent : public UTaggedAssetBrowserFilterBase
{
	GENERATED_BODY()

public:

	UE_API virtual void InitializeInternal(const FTaggedAssetBrowserContext& InContext) override;
	UE_API virtual bool ShouldFilterAssetInternal(const FAssetData& InAssetData) const override;
	UE_API virtual FSlateIcon GetIcon() const override;

	UE_API bool IsAssetRecent(const FAssetData& AssetCandidate) const;

	TAttribute<const FMainMRUFavoritesList*> ListAttribute;
};

/** Filter for assets based on directories. */
UCLASS(MinimalAPI, DisplayName="Directories")
class UTaggedAssetBrowserFilter_Directories : public UTaggedAssetBrowserFilterBase
{
	GENERATED_BODY()

public:

	UE_API virtual FString ToString() const override;
	virtual TOptional<FString> GetInstanceIdentifier() const override { return ToString(); }
	UE_API virtual void ModifyARFilterInternal(FARFilter& Filter) const override;
	UE_API virtual FText GetTooltip() const override;
	UE_API virtual FSlateIcon GetIcon() const override;

	UPROPERTY(EditAnywhere, Category="Filter", meta=(ContentDir))
	TArray<FDirectoryPath> DirectoryPaths;

	/** Name for this directory filter, by default will use first directory name if not set */
	UPROPERTY(EditAnywhere, Category = "Filter")
	FName FilterName;
};

/** Filter for assets based on class. By default will use first class as an icon. */
UCLASS(MinimalAPI, DisplayName="Classes")
class UTaggedAssetBrowserFilter_Class : public UTaggedAssetBrowserFilterBase
{
	GENERATED_BODY()

public:

	UE_API virtual FString ToString() const override;
	virtual TOptional<FString> GetInstanceIdentifier() const override { return ToString(); }
	UE_API virtual FText GetTooltip() const override;
	UE_API virtual FSlateIcon GetIcon() const override;
	
	UE_API virtual void ModifyARFilterInternal(FARFilter& Filter) const override;

	UPROPERTY(EditAnywhere, Category="Filter", meta=(AllowAbstract))
	TArray<TObjectPtr<UClass>> Classes;

	/** True if we should allow child classes in our search. */
	UPROPERTY(EditAnywhere, Category = "Filter")
	bool bAllowChildClasses = true;

	/** Name for this class filter, by default will use first class in list as name if not set */
	UPROPERTY(EditAnywhere, Category = "Filter")
	FName FilterName;
};

#undef UE_API
