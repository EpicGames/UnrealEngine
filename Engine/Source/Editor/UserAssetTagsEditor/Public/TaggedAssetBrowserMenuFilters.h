// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataHierarchyViewModelBase.h"
#include "FrontendFilterBase.h"
#include "TaggedAssetBrowserMenuFilters.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

/** A helper struct passed into filters for initialization, since some filters require contextual knowledge to filter correctly. */
struct FTaggedAssetBrowserContext
{
	DECLARE_DELEGATE_RetVal(const class UTaggedAssetBrowserSection*, FOnGetActiveSection)
	DECLARE_DELEGATE_RetVal(TArray<const class UTaggedAssetBrowserFilterBase*>, FOnGetSelectedFilters)

	TWeakPtr<class STaggedAssetBrowser> TaggedAssetBrowser;
	FOnGetActiveSection OnGetActiveSectionDelegate;
	FOnGetSelectedFilters OnGetSelectedFilters;
};

/** How to resolve extension filter merging */
UENUM()
enum class ETaggedAssetBrowserFilterMergeBehavior
{
	Append, // Add Filter at end
	Insert, // Insert before or after a filter by name
};

/** How to insert a filter */
UENUM()
enum class ETaggedAssetBrowserFilterInsertBehavior
{
	Before,
	After
};

/** Data relevant for extension filters. */
USTRUCT()
struct FTaggedAssetBrowserFilterData_Extension 
{
	GENERATED_BODY()

public:

	/** How this filter should be merged */
	UPROPERTY(EditAnywhere, Category = "Filter")
	ETaggedAssetBrowserFilterMergeBehavior MergeBehavior = ETaggedAssetBrowserFilterMergeBehavior::Append;

	/** How this filter should be inserted */
	UPROPERTY(EditAnywhere, Category = "Filter", meta = (EditCondition = "MergeBehavior==ETaggedAssetBrowserFilterMergeBehavior::Insert", EditConditionHides))
	ETaggedAssetBrowserFilterInsertBehavior InsertBehavior = ETaggedAssetBrowserFilterInsertBehavior::After;

	/** Name for filter this filter should be inserted relative to, assuming this is from / in an extension. */
	UPROPERTY(EditAnywhere, Category = "Filter", meta = (EditCondition = "MergeBehavior==ETaggedAssetBrowserFilterMergeBehavior::Insert", EditConditionHides))
	FName InsertFilterName;
};

/** A filter is the base element of the filter hierarchy built in the TaggedAssetBrowserConfiguration asset.
 *  It filters using two methods:
 *  - ModifyARFilter, which modifies the filter used to query the Asset Registry (override the Internal function for your implementation)
 *  - ShouldFilterAsset, which should return true if you want to remove an asset from display. This is for filtering beyond what the ARFilter allows, such as 'Recent' assets.
 *
 *  While filters are generally part of the hierarchy, sections can also specify additional filters
 */
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UTaggedAssetBrowserFilterBase : public UHierarchyItem
{
	GENERATED_BODY()

public:

	/** Data for this filter that is only needed when we are in an extension. */
	UPROPERTY(EditAnywhere, Category = "Filter", meta = (EditCondition = "IsExtensionFilter", EditConditionHides))
	FTaggedAssetBrowserFilterData_Extension ExtensionData;

public:
	/** Check outer for if we are a filter in an extension config */
	UFUNCTION()
	bool IsExtensionFilter() const;

public:	
	/** The filter has a chance to initialize itself with contextual data to actually work. */
	UE_API void Initialize(const FTaggedAssetBrowserContext& InContext);

	UE_API FName GetIdentifier() const;

	/** The string to display for this filter. */
	virtual FString ToString() const override { return GetClass()->GetDisplayNameText().ToString(); }
	/** The tooltip to display for this filter. */
	virtual FText GetTooltip() const { return GetClass()->GetToolTipText(); }
	/** The optional icon to display for this filter. */
	UE_API virtual FSlateIcon GetIcon() const;
	/** The optional widgets to display for this filter's row. */
	virtual void CreateAdditionalWidgets(TSharedPtr<SHorizontalBox> ExtensionBox) { }
	
	UE_API const FSlateBrush* GetIconBrush() const;

	/** How this filter should modify the AR filter, when selected. */
	UE_API void ModifyARFilter(FARFilter& Filter) const;
	/** An additional check for each asset that passes the AR filter. */
	UE_API bool ShouldFilterAsset(const FAssetData& InAssetData) const;
	
	/** Used for search purposes. Override this to add your own implementation for your custom filter. */
	UE_API virtual bool DoesFilterMatchTextQuery(const FText& Text);

	/** If this filter is selected in the Tagged Asset Browser. */
	UE_API bool IsSelectedFilter() const;
private:
	/** Implementation of your own initialization when the Tagged Asset Browser starts using this filter. */
	virtual void InitializeInternal(const FTaggedAssetBrowserContext& InContext) { }
	
	/** Optional identifier if you are likely to have multiple filters of the same type. Will use only the class name as identifier if not implemented. */
	virtual TOptional<FString> GetInstanceIdentifier() const { return TOptional<FString>(); }

	/** Implementation of your own AR Filter modification. */
	virtual void ModifyARFilterInternal(FARFilter& Filter) const {}
	/** Implementation of your own ShouldFilterAsset function. */
	virtual bool ShouldFilterAssetInternal(const FAssetData& InAssetData) const { return false; }

protected:
	/** Will be valid during usage in a Tagged Asset Browser. */
	TOptional<FTaggedAssetBrowserContext> ActiveContext;
};

uint32 GetTypeHash(const UTaggedAssetBrowserFilterBase& Filter);

#undef UE_API
