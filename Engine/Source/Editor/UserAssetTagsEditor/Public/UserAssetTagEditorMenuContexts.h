// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "UserAssetTagEditorMenuContexts.generated.h"

UCLASS(MinimalAPI)
class UUserAssetTagEditorContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<class SUserAssetTagsEditor> UserAssetTagsEditor;
};

UCLASS(MinimalAPI)
class UTaggedAssetBrowserMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<class STaggedAssetBrowser> TaggedAssetBrowser;
};