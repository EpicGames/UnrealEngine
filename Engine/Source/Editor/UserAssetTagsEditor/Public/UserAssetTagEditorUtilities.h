// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"

class UTaggedAssetBrowserConfiguration;
struct FHierarchyElementViewModel;
struct FToolMenuContext;
class FMetaData;

#define UE_API USERASSETTAGSEDITOR_API

namespace UE
{
	namespace UserAssetTags
	{		
		/** --- User Asset Tags API for adding, removing, querying User Asset Tags --- */

		/** Adding and removing requires the object to be loaded.
		 *  Querying for a tag or all tags can happen on unloaded assets via FAssetData, which checks the Asset Registry. */
		
		UE_API bool HasUserAssetTag(UObject* Object, FName UserAssetTag);
		UE_API bool HasUserAssetTag(const FAssetData& InAssetData, FName UserAssetTag);
		
		UE_API void AddUserAssetTag(UObject* Object, FName UserAssetTag);
		UE_API bool RemoveUserAssetTag(UObject* Object, FName UserAssetTag);

		/** Querying all user asset tags is relatively slow since we construct the list of tags by parsing prefixes. */
		UE_API TArray<FName> GetUserAssetTagsForObject(const UObject* InObject, bool bWithPrefix = false);
		UE_API TArray<FName> GetUserAssetTagsForAssetData(const FAssetData& InAssetData);
		UE_API TArray<FName> GetUserAssetTagsFromMetaData(const FMetaData& MetaData, bool bWithPrefix = false);

		/** -------------------------------------------------------------------------- */
		
		/** The prefix used for User Asset Tags in both FMetaData and the Asset Registry.
		 *  For display purposes we remove the prefix again. */
		inline FString UAT_METADATA_PREFIX = "UAT.";

		/** Ensures the User Asset Tag is properly prefixed. */
		FName GetUATPrefixedTag(FName InUserAssetTag);

		/** Ensures the User Asset Tag is without prefix. */
		FName GetUATWithoutPrefix(FName InUserAssetTag);

		/** Summons the user asset tags editor by invoking its tab. */
		void SummonUserAssetTagsEditor();

		/** Helper function to find the matching standalone configuration asset for a given extension asset. */
		TArray<FAssetData> FindStandaloneConfigurationForExtension(const UTaggedAssetBrowserConfiguration* InExtensionAsset);
		
		/** Helper function to find all profile names for standalone configuration assets. */
		TArray<FName> FindAllStandaloneConfigurationAssetProfileNames();
	}
}

#undef UE_API
