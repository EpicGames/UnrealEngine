// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"

class UContentBrowserAssetContextMenuContext;
struct FAssetData;
class FReply;

namespace UE::UserAssetTags::Menus
{
	void RegisterMenusAndProfiles();

	/** Registers a runtime profile for the AssetViewOptions under the name "TaggedAssetBrowser" to hide entries we don't need to display */
	void RegisterDefaultTaggedAssetBrowserViewOptionsProfile();
	
	/** Extends the Content Browser context menu by adding the ManageTags UI action. */
	void ExtendContentBrowserAssetContextMenu();

	struct FMultipleAssetsTagSingleTagInfo
	{
		TArray<FAssetData> AssetsOwningThisTag;
	};
	struct FMultipleAssetsTagInfo
	{
		TMap<FName, FMultipleAssetsTagSingleTagInfo> TagInfos;
	};

	FMultipleAssetsTagInfo GatherInfoAboutTags(const TArray<FAssetData>& Assets);
	TOptional<FText> DetermineInfoText(const FMultipleAssetsTagSingleTagInfo& SingleTagInfo, const FMultipleAssetsTagInfo& TotalInfo);

	void AddTagsToAssets(FName InUserAssetTag, const TArray<FAssetData>& InAssetData);
	void RemoveTagsFromAssets(FName InUserAssetTag, const TArray<FAssetData>& InAssetData);

	ECheckBoxState DetermineTagUIStatus(FName InUserAssetTag, const TArray<FAssetData>& InAssetData);
	
	struct FMenuRowWidgetConfiguration
	{
		bool bAllowAdd = false;
		bool bAllowRemoval = false;
		TArray<FAssetData> ContextAssets;
	};
}
