// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/SharedPointer.h"
#include "UserAssetTagProvider.generated.h"

#define UE_API USERASSETTAGSEDITOR_API

class SHorizontalBox;
class UUserAssetTagEditorContext;
class SWidget;

UENUM()
enum class EUserAssetTagProviderMenuType
{
	Section,
	SubMenu
};

/**
 *  To add your own source of tags, inherit from this.
 *  Specify a Display Name for UI purposes. Suggested Tags will get displayed in the Tag Editor.
 *  A Tag Provider can override functions to add custom UI elements to the Tag Editor that help with managing tags.
 *  E.g.: A 'Favorites' tag provider can add a 'favorite/unfavorite' toggle to individual tag rows.
 */
UCLASS(MinimalAPI, Transient, Abstract)
class UUserAssetTagProvider : public UObject
{
	GENERATED_BODY()

public:
	struct FResultWithUserFeedback
	{
		FResultWithUserFeedback(bool bInCanPerform) : bResult(bInCanPerform) {}
		
		bool bResult = false;
		/** A message that can be used for user feedback. */
		TOptional<FText> UserFeedback;

		bool operator==(const bool& bOther) const
		{
			return bResult == bOther;
		}

		bool operator!=(const bool& bOther) const
		{
			return !(*this==bOther);
		}
	};

	struct FSuggestedUserAssetTags
	{
		/** If true, will contract to a submenu. Useful if */
		bool bUseSubMenu = false;
		TSet<FName> UserAssetTags;
	};

	/** By default we return the class display name text. */
	UE_API virtual FText GetDisplayNameText(const UUserAssetTagEditorContext* Context) const;
	/** By default we return the class tooltip text. */
	UE_API virtual FText GetToolTipText(const UUserAssetTagEditorContext* Context) const;
	
	/** Returns a list of suggested user asset tags. */
	virtual TSet<FName> GetSuggestedUserAssetTags(const UUserAssetTagEditorContext* Context) const PURE_VIRTUAL(UUserAssetTagProvider::GetSuggestedUserAssetTags, return {}; )

	/** When this provider is enabled, it will automatically show up in the tag editor. It can be disabled in the UI after. */
	virtual bool IsEnabledByDefault() const { return true; }
	
	/** If a provider is only valid for certain assets depending on some conditions, check for it here. If invalid, it can't be displayed. */
	virtual FResultWithUserFeedback IsValid(const UUserAssetTagEditorContext* Context) const { return true; }
	
	/** The menu type, if enabled and valid, how this provider should be presented by default. The type can still be changed in the view options.
	 *  If it's expected to show a large amount of tags, consider returning menu instead of section. */
	virtual EUserAssetTagProviderMenuType GetMenuTypeByDefault() const { return EUserAssetTagProviderMenuType::Section; }
	
	/** Lets you add additional widgets to the row containing a suggested tag in the User Asset Tag Editor */
	virtual TSharedPtr<SWidget> AddAdditionalSuggestedWidgets(FName UserAssetTag, const UUserAssetTagEditorContext* Context) const { return nullptr; }

	/** Lets you add additional widgets to the row containing an owned tag in the User Asset Tag Editor*/
	virtual TSharedPtr<SWidget> AddAdditionalOwnedTagWidgets(FName UserAssetTag, const UUserAssetTagEditorContext* Context) const { return nullptr; }

	/** Lets you add additional */
	virtual void AddToolbarMenuEntries(class UToolMenu* DynamicMenu, const UUserAssetTagEditorContext* Context) const { }
};

#undef UE_API
