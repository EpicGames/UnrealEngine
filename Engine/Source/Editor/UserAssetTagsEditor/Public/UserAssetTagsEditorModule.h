// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataHierarchyViewModelBase.h"
#include "Modules/ModuleManager.h"
#include "Delegates/DelegateCombinations.h"

#define UE_API USERASSETTAGSEDITOR_API

struct FHierarchyElementViewModel;
class UTaggedAssetBrowserFilterBase;
class SDockTab;
class FSpawnTabArgs;
class FAssetRegistryTagsContext;

USERASSETTAGSEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LogUserAssetTags, Log, All);

class FUserAssetTagsEditorModule : public IModuleInterface
{
public:
	DECLARE_DELEGATE_RetVal_ThreeParams(TSharedPtr<FHierarchyElementViewModel>, FOnGetViewModelFactory, UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InViewModel);

	UE_API FUserAssetTagsEditorModule();
	UE_API virtual ~FUserAssetTagsEditorModule() override;
	
	static UE_API FUserAssetTagsEditorModule& Get();
	
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	/** To register a custom view model for display in the Tagged Asset Browser. */
	UE_API void RegisterConfigurationHierarchyElementViewModel(TSubclassOf<UHierarchyElement> FilterType, FOnGetViewModelFactory InFactory);
	UE_API void UnregisterConfigurationHierarchyElementViewModel(TSubclassOf<UHierarchyElement> FilterType);
	const TMap<TSubclassOf<UHierarchyElement>, FOnGetViewModelFactory>& GetConfigurationHierarchyViewModelFactories() const { return ConfigurationHierarchyViewModelFactories; }

	static UE_API FName ManageTagsTabId;
private:
	UE_API void RegisterDefaultConfigurationHierarchyElementViewModels();
	UE_API void RegisterDetailsCustomizations();

	/** We extend the Content Browser command list so that the shortcut works. */
	UE_API void OnRegisterCommandList(FName ContextName, TSharedRef<FUICommandList> CommandList);

	static UE_API TSharedRef<SDockTab> SpawnUserAssetTagsEditorNomadTab(const FSpawnTabArgs& SpawnTabArgs);

	static UE_API void GetAssetRegistryTagsFromUserAssetTags(FAssetRegistryTagsContext AssetRegistryTagsContext);
	static UE_API void TransferUserAssetTagsFromOldPackageToNewPackage(const FMetaData& OldMetaData, FMetaData& NewMetaData, const UPackage* OldPackage, UPackage* NewPackage);

private:
	TMap<TSubclassOf<UHierarchyElement>, FOnGetViewModelFactory> ConfigurationHierarchyViewModelFactories;
	TSharedPtr<FUICommandList> CommandList;
	FDelegateHandle GetAssetRegistryTagsFromUserAssetTagsDelegateHandle;
	FDelegateHandle TransferUserAssetTagsFromOldPackageToNewPackageDelegateHandle;
};

#undef UE_API
