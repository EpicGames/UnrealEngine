// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MRUFavoritesList.h"
#include "SAssetEditorViewport.h"
#include "STaggedAssetBrowserContent.h"
#include "TaggedAssetBrowserMenuFilters.h"
#include "Editor/EditorEngine.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "TaggedAssetBrowserFilters/TaggedAssetBrowser_CommonFilters.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/STreeView.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"

#define UE_API USERASSETTAGSEDITOR_API

class SAssetPicker;
class UTaggedAssetBrowserMenuContext;
struct FAssetPickerConfig;

DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateAssetWidget, const FAssetData& Asset)

/** You can provide custom tabs to be displayed in the right column. Useful if you want to add custom content. */
struct FTaggedAssetBrowserCustomTabInfo
{
	FTaggedAssetBrowserCustomTabInfo() {};
	
	/** The title displayed in the browser. */
	FText Title;

	/** The icon displayed next to the title. Optional. */
	const FSlateBrush* Icon = nullptr;
	
	/** The content of the tab to be displayed. */
	FOnGetContent OnGetTabContent;
	
	/** Will be called when an asset gets selected so the custom tab content can react accordingly. */
	FOnAssetSelected OnAssetSelected;
};

struct FDefaultDetailsTabConfiguration
{
	FDefaultDetailsTabConfiguration() {};
	
	bool bUseDefaultDetailsTab = true;
	TOptional<FText> EmptySelectionMessage;
	FOnGenerateAssetWidget OnGenerateAssetThumbnailOverrideWidgetDelegate;
};

class STaggedAssetBrowser : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(TArray<TSharedRef<FFrontendFilter>>, FOnGetExtraFrontendFilters)
	DECLARE_DELEGATE_OneParam(FOnOverrideAssetPickerConfig, FAssetPickerConfig&)
	DECLARE_DELEGATE_OneParam(FOnActiveSectionChanged, const UTaggedAssetBrowserSection*)
	DECLARE_DELEGATE(FOnPrimaryFilterSelectionChanged)

	/** Contains override names for specific menu names. By default, the ProfileName is used as suffix for all menus. */
	struct FInterfaceOverrideProfiles
	{
		TOptional<FName> AssetViewOptionsProfileName;
		TOptional<FName> AssetViewContextMenuName;
		TOptional<FName> FilterBarSaveName;
		TOptional<EAssetTypeCategories::Type> DefaultFilterMenuExpansion;
	};
	
	SLATE_BEGIN_ARGS(STaggedAssetBrowser)
		: _RecentAndFavoritesList(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetRecentlyOpenedAssets())
		, _AssetSelectionMode(ESelectionMode::Single)
		{
		}
		/** The asset types to display. */
		SLATE_ARGUMENT(TArray<UClass*>, AvailableClasses)
		/** The configuration for the default details tab. You can remove it entirely using FDefaultDetailsTabConfiguration::bUseDefaultDetailsTab. */
		SLATE_ARGUMENT(FDefaultDetailsTabConfiguration, DefaultDetailsTabConfiguration)
		/** If specified, will add these tabs in the details area via segmented controls. */
		SLATE_ARGUMENT(TArray<FTaggedAssetBrowserCustomTabInfo>, CustomTabInfos)
		/** If you wish to use specific overrides of menus, you can specify them in the InterfaceOverrideProfiles.
		 * If not specified, the ProfileName will be used for all menu names. */
		SLATE_ARGUMENT(FInterfaceOverrideProfiles, InterfaceOverrideProfiles)
		/** An externally owned recent & favorites list. Defaults to the editor's main list, but can be a custom list as well. */
		SLATE_ATTRIBUTE(const FMainMRUFavoritesList*, RecentAndFavoritesList)
		/** The property handle for which this browser is summoned. Used for contextual filtering. In wizard windows or similar, will be nullptr. */
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)
		/** Additional assets used for contextual filtering. In wizard windows or similar, construct FAssetData pointing at the relevant path. */
		SLATE_ARGUMENT(TArray<FAssetData>, AdditionalReferencingAssets)
		
		/** Additional Content Browser filters (in the dropdown) can specified here. */
		SLATE_EVENT(FOnGetExtraFrontendFilters, OnGetExtraFrontendFilters)
		/** Allows you to dynamically change the 'Add Filters' menu to remove or add desired items. */
		SLATE_EVENT(FOnExtendAddFilterMenu, OnExtendAddFilterMenu)
		/** An additional, always active filter */
		SLATE_EVENT(FOnShouldFilterAsset, OnAdditionalShouldFilterAsset)

		/**
		 * Generic Callback for editors to customize our asset picker config before we create it. So that we don't have to update our API every time the asset picker changes.
		 * Use with care, as we do not guarantee STaggedAssetBrowser functionality will work depending on customizations done. 
		 * Consider capturing existing values if previous functionality may be important to you.
		 */
		SLATE_EVENT(FOnOverrideAssetPickerConfig, OnOverrideAssetPickerConfig)
		
		/** If supplied, will generate a custom tooltip when hovering over an asset. */
		SLATE_EVENT(FOnGenerateAssetWidget, OnGenerateAssetTooltipWidget)
		
		/** If supplied, determines whether an asset should get a custom tooltip. Requires OnGenerateAssetTooltipWidget to be bound. */
		SLATE_EVENT(FOnIsAssetValidForCustomToolTip, OnIsAssetValidForCustomToolTip)

		/** Determines the asset selection mode; single asset selection by default. */
		SLATE_ARGUMENT(ESelectionMode::Type, AssetSelectionMode)

		/** The event that triggers when an asset has been selected. */
		SLATE_EVENT(FOnAssetSelected, OnAssetSelected)

		/** The event that triggers when an asset has been activated (double clicked, custom buttons to proceed etc.). */
		SLATE_EVENT(FOnAssetsActivated, OnAssetsActivated)
		
		/** An additional widget to attach below the asset browser. */
		SLATE_ARGUMENT(TSharedPtr<SWidget>, AdditionalBottomWidget)

		/** Called when the active section changes. Fired after internal state has been updated. */
		SLATE_EVENT(FOnActiveSectionChanged, OnActiveSectionChanged)

		/** Called when the primary filter tree selection changes. */
		SLATE_EVENT(FOnPrimaryFilterSelectionChanged, OnPrimaryFilterSelectionChanged)
	SLATE_END_ARGS()

	
	UE_API void Construct(const FArguments& InArgs, const UTaggedAssetBrowserConfiguration& InTaggedAssetBrowserConfiguration);

	UE_API STaggedAssetBrowser();
	UE_API virtual ~STaggedAssetBrowser() override;

	UE_API void RefreshBackendFilter() const;

	UE_API TArray<FAssetData> GetSelectedAssets() const;
	
	UE_API TArray<UClass*> GetDisplayedAssetTypes() const;

	UE_API TAttribute<const FMainMRUFavoritesList*> GetFavoritesListAttribute() const;

	/** Returns the currently active section, or nullptr if no section is active. */
	UE_API const UTaggedAssetBrowserSection* GetActiveSection() const;

	/** Returns the asset picker widget inside this browser, if it exists */
	UE_API TSharedPtr<SAssetPicker> GetAssetPicker() const;

	/** Returns the currently selected primary filter items. */
	UE_API TArray<const UTaggedAssetBrowserFilterBase*> GetSelectedPrimaryFilters() const;

	/** Returns all primary filter items in the current section, regardless of selection. */
	UE_API TArray<const UTaggedAssetBrowserFilterBase*> GetAllPrimaryFilters() const;

private:
	void InitializeValues();

	void RefreshPrimaryFilterRootItems(bool bRefreshWidget = true);
	
	void ApplyFilterExtensions();

	void InitAssetContextMenu();
	void CheckValidSetup() const;
	
	/** This function should take into account all the different widgets states that could affect the FARFilter. */
	FARFilter CreateCurrentBackendFilter() const;
	
	bool ShouldFilterAsset(const FAssetData& AssetData) const;
	bool NeedsCustomTabArea() const;

	void PopulateSectionsSlot();
	void PopulatePrimaryFiltersSlot();
	void PopulateAssetBrowserSplitterSlot();
	void PopulateAssetBrowserContentSlot();
	void PopulateCustomTabsSlot();

	void OnPrimaryFiltersSlotResized(float InSize);
	float OnGetPrimaryFiltersSlotMinSize() const;
	float OnGetPrimaryFiltersSlotSizeValue() const;

	/** We initialize the duplicated filters contained in the filter root of the asset so that the filters have context available. */
	void InitializeFilters();

private:
	/** This has to be called whenever any kind of filtering changes and refreshes the actual asset view */
	void OnFilterChanged() const;

	struct FFilterPath
	{
		TArray<TWeakObjectPtr<UHierarchyElement>> Path;

		TWeakObjectPtr<UHierarchyElement> GetEntry() const
		{
			TWeakObjectPtr<UHierarchyElement> Entry = nullptr;
			
			if(Path.Num() > 0)
			{
				Entry = Path[Path.Num() - 1];
			}
			
			return Entry;
		}

		bool operator==(const FFilterPath& Item) const
		{
			return Path == Item.Path;
		}
	};

	/** Search functionality */
	void OnFilterSearchTextChanged(const FText& Text);
	void OnSearchButtonClicked(SSearchBox::SearchDirection SearchDirection);
	void OnFilterSearchTextCommitted(const FText& Text, ETextCommit::Type Arg);
	TOptional<SSearchBox::FSearchResultData> GetSearchResultData() const;
	void GenerateSearchItems(TWeakObjectPtr<UHierarchyElement> Root, TArray<TWeakObjectPtr<UHierarchyElement>> ParentChain, TArray<FFilterPath>& OutSearchItems) const;
	void ExpandPrimaryFilterSearchResults();
	void SelectNextPrimaryFilterSearchResult();
	void SelectPreviousPrimaryFilterSearchResult();

	/** Sections functionality */
	void OnSectionSelected(const UTaggedAssetBrowserSection* InTaggedAssetBrowserSection);

	void OnGetChildFiltersForFilter(TObjectPtr<UHierarchyElement> PrimaryFilter, TArray<TObjectPtr<UHierarchyElement>>& OutChildren) const;
	bool OnCompareFiltersForEquality(const UTaggedAssetBrowserFilterBase& PrimaryFilterA, const UTaggedAssetBrowserFilterBase& PrimaryFilterB) const;
	
	TSharedRef<ITableRow> GenerateWidgetRowForPrimaryFilter(TObjectPtr<UHierarchyElement> PrimaryFilter, const TSharedRef<STableViewBase>& OwningTable) const;
	
	void OnAssetSelected(const FAssetData& AssetData);
	void OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type) const;
	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& AssetData) const;
	TSharedRef<SToolTip> OnGetCustomAssetTooltip(FAssetData& AssetData);
	void OnPrimaryFilterSelected(TObjectPtr<UHierarchyElement> Filter, ESelectInfo::Type Arg);

	TArray<TSharedRef<FFrontendFilter>> OnGetExtraFrontendFilters() const;
	
	void OnAssetBrowserConfigPropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent);
	
	void SaveSettings() const;
	void LoadSettings();

	TArray<UTaggedAssetBrowserConfiguration*> GetExtensionAssets() const;
	
	FName GetAssetContextMenuName() const;
private:
	TWeakObjectPtr<const UTaggedAssetBrowserConfiguration> Configuration;
	
	TArray<UClass*> AvailableClasses;
	TStrongObjectPtr<UTaggedAssetBrowserFilterRoot> FilterHierarchyRoot;
	FDefaultDetailsTabConfiguration DefaultDetailsTabConfiguration;
	TArray<FTaggedAssetBrowserCustomTabInfo> CustomTabInfos;
	TAttribute<const FMainMRUFavoritesList*> RecentAndFavoritesList;
	TOptional<FName> TaggedBrowserProfileName;
	FInterfaceOverrideProfiles InterfaceOverrideProfiles;
	FOnAssetSelected OnAssetSelectedDelegate;
	FOnGenerateAssetWidget OnGenerateAssetTooltipWidgetDelegate;
	FOnIsAssetValidForCustomToolTip OnIsAssetValidForCustomToolTipDelegate;
	FOnAssetsActivated OnAssetsActivatedDelegate;
	FOnGetExtraFrontendFilters OnGetExtraFrontendFiltersDelegate;
	FOnExtendAddFilterMenu OnExtendAddFilterMenuDelegate;
	FOnShouldFilterAsset OnAdditionalShouldFilterAssetDelegate;
	FOnOverrideAssetPickerConfig OnOverrideAssetPickerConfigDelegate;
	FOnActiveSectionChanged OnActiveSectionChangedDelegate;
	FOnPrimaryFilterSelectionChanged OnPrimaryFilterSelectionChangedDelegate;
	ESelectionMode::Type AssetSelectionMode = ESelectionMode::Single;
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TArray<FAssetData> AdditionalReferencingAssets;

	TArray<FFilterPath> SourceSearchResults;
	TOptional<FFilterPath> FocusedSearchResult;
private:
	TArray<TObjectPtr<UHierarchyElement>> PrimaryFilterRootItems;
	TSharedPtr<STreeView<TObjectPtr<UHierarchyElement>>> PrimaryFilterSelector;
	TSharedPtr<STaggedAssetBrowserContent> AssetBrowserContent;

	SSplitter::FSlot* PrimaryFiltersSlot = nullptr;
	SHorizontalBox::FSlot* SectionsSlot = nullptr;
	SHorizontalBox::FSlot* AssetBrowserAreaSplitterSlot = nullptr;
	SSplitter::FSlot* AssetBrowserContentSlot = nullptr;
	SSplitter::FSlot* CustomTabsSlot = nullptr;
	SVerticalBox::FSlot* AdditionalWidgetSlot = nullptr;

	TSharedPtr<class STaggedAssetBrowserSections> SectionsSlotWidget;
	TSharedPtr<SWidget> PrimaryFiltersSlotWidget;

	TStrongObjectPtr<UTaggedAssetBrowserMenuContext> ThisContext;

	struct FDefaultDetailsTabMembers : public TSharedFromThis<FDefaultDetailsTabMembers>
	{
		TSharedPtr<SWidgetSwitcher> DefaultDetailsSwitcher;
		SWidgetSwitcher::FSlot* AssetBrowserDefaultDetailsSlot = nullptr;

		EVisibility OnGetThumbnailVisibility() const;
	};

	/** A struct encapsulating members who are only relevant if the default details tab is created. */
	TSharedPtr<FDefaultDetailsTabMembers> DefaultDetailsTabMembers;
	
	/** Used to temporarily stop saving & loading config state.
	 * This is used for initialization where setting the default state should not be saved into config.*/
	bool bSuppressSaveAndLoad = false;
	
	/** We use this fallback as a means to save our last selected filter, in case we have no valid active selections anymore. */
	mutable FName LastSelectedPrimaryFilterIdentifierFallback = NAME_None;

	struct FSlotValues
	{
		FSlotValues() = default;

		float CurrentSizeValue = 0.15f;
		
		float DefaultSizeValue = 0.15f;
		float DefaultMinSize = 50.f;
	};
	
	mutable FSlotValues PrimaryFiltersSlotValues;
};

class STaggedAssetBrowserWindow : public SWindow
{
public:
	SLATE_BEGIN_ARGS(STaggedAssetBrowserWindow)
		{
		}
		SLATE_ARGUMENT(STaggedAssetBrowser::FArguments, AssetBrowserArgs)
		SLATE_ARGUMENT(SWindow::FArguments, WindowArgs)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const UTaggedAssetBrowserConfiguration& Configuration);

	STaggedAssetBrowserWindow() {}
	virtual ~STaggedAssetBrowserWindow() override {}
	
	UE_API bool HasSelectedAssets() const;
	
	UE_API TArray<FAssetData> GetSelectedAssets() const;
	
protected:	
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	
protected:
	TSharedPtr<STaggedAssetBrowser> AssetBrowser;
};

#undef UE_API
