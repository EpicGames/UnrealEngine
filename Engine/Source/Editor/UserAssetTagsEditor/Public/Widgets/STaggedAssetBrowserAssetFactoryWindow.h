// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "STaggedAssetBrowser.h"
#include "Factories/Factory.h"

#define UE_API USERASSETTAGSEDITOR_API

/** This window is meant to be used within a factory.
 *  Modal: Summon this window from ConfigureProperties, then check "ShouldProceedWithAction", retrieve selected assets and settings.
 *  Non-Modal: Summon this window from ConfigurePropertiesAsync and pass in the AsyncFactoryArguments.
 *  In both cases, the actual asset creation invocation of the factory is handled by the Content Browser or the Asset Tools.
 */
class STaggedAssetBrowserAssetFactoryWindow : public STaggedAssetBrowserWindow
{
public:
	struct FAsyncFactoryArguments
	{
		/** The factory that summoned this window from ConfigurePropertiesAsync. */
		TStrongObjectPtr<UFactory> Factory;
		/** A callback to let your factory be informed of asset activation so it can configure its properties accordingly. */
		FOnAssetsActivated OnAssetsActivated;
		/** The callbacks from ConfigurePropertiesAsync to let the process know to continue or to cancel. */
		FOnFactoryConfigurePropertiesAsyncComplete OnFactoryConfigurePropertiesComplete;
		FOnFactoryConfigurePropertiesAsyncCancelled OnFactoryConfigurePropertiesCancelled;
	};
	
	SLATE_BEGIN_ARGS(STaggedAssetBrowserAssetFactoryWindow)	{}
		SLATE_ARGUMENT(STaggedAssetBrowserWindow::FArguments, AssetBrowserWindowArgs)
		/** Async callbacks for use within an async factory */
		SLATE_ARGUMENT(TOptional<FAsyncFactoryArguments>, AsyncFactoryArguments)
		/** An additional factory settings class can be specified for additional factory information. */
		SLATE_ARGUMENT_DEFAULT(UClass*, AdditionalFactorySettingsClass) = nullptr;
		/** If the asset definition is used, data such as its AssetDisplayName will be used. If not, the class display name is used. */
		SLATE_ARGUMENT_DEFAULT(bool, bUseAssetDefinition) = true;
		/** If empty asset creation is allowed, a "Create Empty" button will show up on the bottom left that will terminate the window without a selection. */
		SLATE_ARGUMENT_DEFAULT(bool, bAllowEmptyAssetCreation) = true;
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const UTaggedAssetBrowserConfiguration& Configuration, UClass& CreatedClass);

	STaggedAssetBrowserAssetFactoryWindow() {}
	UE_API virtual ~STaggedAssetBrowserAssetFactoryWindow() override;

	/** For use in a modal context: retrieve whether the user activated assets or clicked a Proceed button, or closed the window.*/
	bool ShouldProceedWithAction() const { return bProceedWithAction; }

	/** Retrieve the factory settings object, if available. This can be used to further customize the factory process. */
	UObject* RetrieveFactorySettings() const { return FactorySettingsObject; }
	
private:
	bool CheckValidArguments(const FArguments& InArgs) const;
	
	/** The function that will be called by our buttons or by the asset picker itself if double-clicking, hitting enter etc. */
	void OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type);
	
	FReply Proceed();
	FReply Cancel();

	FText GetAssetTypeName() const;
	
	FText GetCreateButtonTooltip() const;
	
	TSharedRef<SWidget> CreateFactorySettingsTab();

	UE_API virtual FString GetReferencerName() const override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	
	void CloseIfOtherWindowFocused(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget);

private:
	TWeakObjectPtr<UClass> CreatedClass;
	TWeakObjectPtr<UClass> AdditionalFactorySettingsClass;
	TWeakObjectPtr<const UAssetDefinition> AssetDefinition;

	TObjectPtr<UObject> FactorySettingsObject;
	TSharedPtr<class IDetailsView> FactorySettingsWidget;
	
	bool bProceedWithAction = false;
	bool bAllowEmptyAssetCreation = false;

	TOptional<FAsyncFactoryArguments> AsyncFactoryArguments;
};

#undef UE_API
