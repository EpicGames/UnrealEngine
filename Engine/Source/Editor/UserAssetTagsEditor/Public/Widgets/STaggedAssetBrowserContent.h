// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrontendFilterBase.h"
#include "Filters/GenericFilter.h"
#include "Filters/SAssetFilterBar.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API USERASSETTAGSEDITOR_API

class SAssetPicker;

class STaggedAssetBrowserContent : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterAsset, const FAssetData&)
	DECLARE_DELEGATE_RetVal(TArray<TSharedRef<FFrontendFilter>>, FOnGetExtraFrontendFilters)

	SLATE_BEGIN_ARGS(STaggedAssetBrowserContent)
		{
		}
		SLATE_ARGUMENT(FAssetPickerConfig, InitialConfig)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	UE_API void SetARFilter(FARFilter InFilter);

	UE_API TArray<FAssetData> GetCurrentSelection() const;

	/** Returns the asset picker widget created during construction, if it exists. */
	UE_API TSharedPtr<SAssetPicker> GetAssetPicker() const;

private:
	FRefreshAssetViewDelegate RefreshAssetViewDelegate;
	FSyncToAssetsDelegate SyncToAssetsDelegate;
	FSetARFilterDelegate SetNewFilterDelegate;
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;

	FAssetPickerConfig InitialAssetPickerConfig;

	TSharedPtr<SAssetPicker> AssetPickerPtr;
};

#undef UE_API
