// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TaggedAssetBrowserFilters/TaggedAssetBrowser_CommonFilters.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API USERASSETTAGSEDITOR_API

class UTaggedAssetBrowserSection;
class UTaggedAssetBrowserFilterRoot;
/**
 * 
 */

class STaggedAssetBrowserSection : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STaggedAssetBrowserSection)
		{}
		SLATE_EVENT(FOnCheckStateChanged, OnCheckStateChanged)
		SLATE_ATTRIBUTE(ECheckBoxState, IsChecked)
		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UTaggedAssetBrowserSection& InSection);

private:
	void OnCheckStateChanged(ECheckBoxState CheckBoxState);
	ECheckBoxState GetCheckState() const;
	TSharedRef<SWidget> OnGetMenuContent();
	FSlateColor GetIconForegroundColor() const;
	const FSlateBrush* GetIconBrush() const;
	EVisibility GetLabelVisibility() const;
	FText GetLabelText() const;

private:
	TWeakObjectPtr<const UTaggedAssetBrowserSection> Section;

	FOnCheckStateChanged OnCheckStateChangedDelegate;
	TAttribute<ECheckBoxState> IsCheckedAttribute;
	FOnGetContent OnGetMenuContentDelegate;
};

class STaggedAssetBrowserSections : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSectionSelected, const UTaggedAssetBrowserSection* Section)
	
	SLATE_BEGIN_ARGS(STaggedAssetBrowserSections)
		: _InitiallyActiveSection(nullptr)
		{}
		SLATE_ARGUMENT(const UTaggedAssetBrowserSection*, InitiallyActiveSection)
		SLATE_EVENT(FOnSectionSelected, OnSectionSelected)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs, const UTaggedAssetBrowserFilterRoot& InFilterRoot);

	UE_API void RebuildWidget();

	const UTaggedAssetBrowserSection* GetActiveSection() const { return ActiveSection.Get(); }

private:
	UE_API void OnSectionSelected(ECheckBoxState CheckBoxState, const UTaggedAssetBrowserSection* InSection);
	UE_API ECheckBoxState IsSectionActive(const UTaggedAssetBrowserSection* InSection) const;

private:
	TWeakObjectPtr<const UTaggedAssetBrowserFilterRoot> FilterRoot;

	FOnSectionSelected OnSectionSelectedDelegate;

	TWeakObjectPtr<const UTaggedAssetBrowserSection> ActiveSection;
	TSharedPtr<SScrollBox> ScrollBox;

};

#undef UE_API
