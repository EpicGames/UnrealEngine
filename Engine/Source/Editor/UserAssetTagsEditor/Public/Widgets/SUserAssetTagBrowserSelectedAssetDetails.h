// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "AssetThumbnail.h"
#include "AssetRegistry/AssetData.h"

#define UE_API USERASSETTAGSEDITOR_API

struct FDisplayedPropertyData
{
	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldDisplayProperty, const FAssetData&)
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FGenerateWidget, const FAssetData&)

	FShouldDisplayProperty ShouldDisplayPropertyDelegate;
	FGenerateWidget NameWidgetDelegate;
	FGenerateWidget ValueWidgetDelegate;
};

struct FAssetDetailsDisplayInfo
{
	DECLARE_DELEGATE_RetVal_OneParam(FText, FGetDescription, const FAssetData&);
	
	FGetDescription GetDescriptionDelegate;
	TArray<FDisplayedPropertyData> DisplayedProperties;
};

struct FTaggedAssetBrowserDetailsDisplayDatabase
{
	USERASSETTAGSEDITOR_API static void RegisterClass(UClass* Class, FAssetDetailsDisplayInfo ClassInfo);
	static TMap<UClass*, FAssetDetailsDisplayInfo> Data;
};

DECLARE_DELEGATE_OneParam(FOnAssetTagActivated, const FName& UserAssetTag);

class SUserAssetTag : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SUserAssetTag)
	{
	}
		SLATE_EVENT(FOnAssetTagActivated, OnAssetTagActivated)
		SLATE_ARGUMENT(TOptional<FText>, OnAssetTagActivatedTooltip)
	SLATE_END_ARGS()

	// TODO (ME) Add Domain Info for global lookup for additional info such as tooltip, description etc
	UE_API void Construct(const FArguments& InArgs, const FName& UserAssetTag);

private:
	UE_API FReply OnClicked() const;

private:
	FName UserAssetTag;
	FOnAssetTagActivated OnAssetTagActivated;
	TOptional<FText> OnAssetTagActivatedTooltip;
};

class SUserAssetTagRow : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SUserAssetTagRow)
		{
		}
		SLATE_EVENT(FOnAssetTagActivated, OnAssetTagActivated)
		SLATE_ARGUMENT(TOptional<FText>, OnAssetTagActivatedTooltip)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FAssetData& Asset);
};

/**
 * The default widget for the right hand side of the tagged asset browser, showing details of the selected assets.
 * It requires previous configuration using the FUserAssetTag
 */
class SUserAssetTagBrowserSelectedAssetDetails : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateWidget, const FAssetData& AssetData)
	
	SLATE_BEGIN_ARGS(SUserAssetTagBrowserSelectedAssetDetails)
		: _ShowThumbnailSlotWidget(EVisibility::Visible)
		, _MaxDesiredDescriptionWidth(300.f)
		, _MaxDesiredPropertiesWidth(300.f)
		{
		}
		SLATE_ATTRIBUTE(EVisibility, ShowThumbnailSlotWidget)
		SLATE_EVENT(FOnGenerateWidget, OnGenerateThumbnailReplacementWidget)
		SLATE_EVENT(FOnAssetTagActivated, OnAssetTagActivated)
		SLATE_ARGUMENT(TOptional<FText>, OnAssetTagActivatedTooltip)
	
		/* Text will automatically wrap according to actual width.
		 * With this you can specify the maximum width you want the description to be; this is not the actual width.
		 * Actual width might be increased due to big asset names.
		 */
		SLATE_ARGUMENT(float, MaxDesiredDescriptionWidth)
		SLATE_ARGUMENT(float, MaxDesiredPropertiesWidth)

	SLATE_END_ARGS()
	
	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs, const FAssetData& Asset);

private:
	UE_API FReply CopyAssetPathToClipboard() const;
	
private:
	FAssetData AssetData;
	TAttribute<EVisibility> ShowThumbnailSlot;
	FOnGenerateWidget OnGenerateThumbnailReplacementWidgetDelegate;
	TSharedPtr<FAssetThumbnail> CurrentAssetThumbnail;
	FOnAssetTagActivated OnAssetTagActivated;
	TOptional<FText> OnAssetTagActivatedTooltip;

	UE_API TSharedRef<SWidget> CreateAssetThumbnailWidget();
	UE_API TSharedRef<SWidget> CreateTitleWidget();
	UE_API TSharedRef<SWidget> CreateTypeWidget();
	UE_API TSharedRef<SWidget> CreatePathWidget();
	UE_API TSharedRef<SWidget> CreateDescriptionWidget();
	UE_API TSharedRef<SWidget> CreateOptionalPropertiesList();
	UE_API TSharedRef<SWidget> CreateAssetTagRow();
};

#undef UE_API
