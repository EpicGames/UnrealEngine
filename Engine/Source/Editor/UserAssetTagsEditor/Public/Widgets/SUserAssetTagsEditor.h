// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UserAssetTagProvider.h"
#include "Widgets/SCompoundWidget.h"
#include "AssetRegistry/AssetData.h"
#include "Templates/SharedPointer.h"
#include "UserAssetTagEditorMenuContexts.h"

#define UE_API USERASSETTAGSEDITOR_API

struct FToolMenuContext;
struct FToolUIAction;
struct FToolMenuCustomWidgetContext;

/** Optional info relating to a specific tag, useful to surface more information to the user. */
struct FUserAssetTagInfo
{
	/** A tag can come from multiple sources at once. */
	TArray<const UUserAssetTagProvider*> Sources;
};

/** The actual editor widget to assign/unassign tags. */
class SUserAssetTagsEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SUserAssetTagsEditor)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual ~SUserAssetTagsEditor() override;
	
	const TArray<FAssetData>& GetSelectedAssets() const { return SelectedAssets; }

	UE_API void RefreshDataAndMenus();
	static UE_API void RefreshMenus();
	static UE_API void RefreshSelectedAssetsMenu();
	static UE_API void RefreshSuggestedTagMenus();
	static UE_API void RefreshOwnedTagMenus();
	static UE_API void RefreshProviderExtensionToolbarMenu();

private:
	UE_API TSharedPtr<SWidget> CreateSelectedAssetList();
	UE_API TSharedPtr<SWidget> CreateSuggestedTagList();
	UE_API TSharedPtr<SWidget> CreateOwnedTagsList();
	UE_API TSharedPtr<SWidget> CreateToolbar();

	static UE_API TArray<FAssetData> GetCurrentContentBrowserAssetSelection();
	
	UE_API void OnCommitNewTag(const FText& Text, ETextCommit::Type CommitType);
	UE_API FReply OnAddTagButtonClicked();
	UE_API bool IsAddTagButtonEnabled() const;

	UE_API const FSlateBrush* GetViewOptionsBadgeIcon() const;
	
	static UE_API TSharedRef<SWidget> GenerateRowContent_SelectedAsset(const FAssetData& AssetData, const UUserAssetTagEditorContext* Context);
	static UE_API TSharedRef<SWidget> GenerateRowContent_SuggestedTag(FName UserAssetTag, const UUserAssetTagEditorContext* Context);
	static UE_API TSharedRef<SWidget> GenerateRowContent_OwnedTag(FName UserAssetTag, const UUserAssetTagEditorContext* Context);
	static UE_API TSharedRef<SWidget> CreateMenuControlWidget(const FToolMenuContext& ToolMenuContext, const FToolMenuCustomWidgetContext& ToolMenuCustomWidgetContext, const UClass* Class);

	UE_API TSharedRef<SWidget> OnGetViewOptions();
	static UE_API void ToggleSortByAlphabet(const FToolMenuContext& ToolMenuContext);
	static UE_API ECheckBoxState GetShouldSortByAlphabet(const FToolMenuContext& ToolMenuContext);
	static UE_API void ToggleViewOption_ProviderClass(const FToolMenuContext& ToolMenuContext, const UClass* ProviderClass);
	static UE_API ECheckBoxState GetViewOptions_IsProviderClassEnabled(const FToolMenuContext& ToolMenuContext, const UClass* Class);

	static UE_API void SetViewOption_ProviderClassMenuType(EUserAssetTagProviderMenuType InMenuType, const UClass* ProviderClass);
	static UE_API EUserAssetTagProviderMenuType GetViewOptions_ProviderClassMenuType(const UClass* ProviderClass);
	
	static UE_API const TArray<const UUserAssetTagProvider*>& GetAllProviderCDOs();
	static UE_API TArray<const UUserAssetTagProvider*> GetAllValidDefaultEnabledProviderCDOs(const UUserAssetTagEditorContext* Context);
	static UE_API TArray<const UUserAssetTagProvider*> GetValidProviderCDOs(const UUserAssetTagEditorContext* Context);
	static UE_API TArray<const UUserAssetTagProvider*> GetEnabledProviderCDOs(const UUserAssetTagEditorContext* Context);

	static UE_API FToolUIAction CreateToggleTagCheckboxAction_SuggestedTag(const FName& InUserAssetTag);
	static UE_API FToolUIAction CreateToggleTagCheckboxAction_OwnedTag(const FName& InUserAssetTag);

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	static UE_API FReply OnDocumentationRequested();
	static UE_API FText GetDocumentationButtonTooltipText();

	static UE_API FReply OnDeleteTagClicked(FName InUserAssetTag, const UUserAssetTagEditorContext* Context);

	UE_API void HandleAssetSelectionChanged(const TArray<FAssetData>& AssetData, bool bIsPrimaryBrowser);

private:
	TSharedPtr<SWidget> SelectedAssetList;
	TSharedPtr<SWidget> TagSuggestionList;
	TSharedPtr<SWidget> OwnedTagsList;

	TArray<FAssetData> SelectedAssets;
	
	TSharedPtr<class SEditableTextBox> AddTagTextBox;
	TSharedPtr<SWidget> ToolbarWidget;

	/** The editor relies on UToolMenus a lot. We construct a context object once, and then pass it around to mostly static functions that generate the UI. */
	TStrongObjectPtr<UUserAssetTagEditorContext> ThisContext;

private:
	static UE_API TArray<const UUserAssetTagProvider*> CachedProviderCDOs;
};

#undef UE_API
