// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class UserAssetTagsEditor : ModuleRules
{
	public UserAssetTagsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"EditorWidgets",
				"UnrealEd", 
				"DataHierarchyEditor"
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"EditorConfig",
				"ContentBrowser",
				"ContentBrowserData",
				"ToolMenus",
				"ToolWidgets",
				"AssetDefinition",
				"Slate",
				"SlateCore",
				"AdvancedPreviewScene",
				"InputCore",
				"ApplicationCore",
				"Kismet",
				"DeveloperSettings",
				"AssetTools"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				"PropertyEditor"
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
